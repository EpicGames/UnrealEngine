// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DataLayerEditor : ModuleRules
{
	public DataLayerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"EditorWidgets",
				"EditorSubsystem",
				"PropertyEditor",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"SceneOutliner",
				"ToolMenus",
				"AssetTools",
				"ContentBrowserData"
			}
		);

		// This module needs AutoRTFM so disable the auto disable since its in a plugin.
		// The call to the delegate `OnObjectPropertyChanged` calls into here.
		bDisableAutoRTFMInstrumentation = false;
	}
}
