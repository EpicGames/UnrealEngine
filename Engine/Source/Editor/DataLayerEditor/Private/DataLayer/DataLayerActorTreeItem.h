// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "SceneOutlinerFwd.h"
#include "ActorTreeItem.h"
#include "Engine/World.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/SecureHash.h"
#include "UObject/ObjectKey.h"

//////////////////////////////////////////////////////////////////////////
// FDataLayerActorTreeItemData

struct FDataLayerActorTreeItemData
{
	FDataLayerActorTreeItemData(AActor* InActor, UDataLayerInstance* InDataLayerInstance)
		: Actor(InActor)
		, DataLayerInstance(InDataLayerInstance)
	{}
	TWeakObjectPtr<AActor> Actor;
	TWeakObjectPtr<UDataLayerInstance> DataLayerInstance;
};

//////////////////////////////////////////////////////////////////////////
// FDataLayerActorTreeItem

struct FDataLayerActorTreeItem : public FActorTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FFilterPredicate, const AActor*, const UDataLayerInstance* InDataLayerInstance);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FInteractivePredicate, const AActor*, const UDataLayerInstance* InDataLayerInstance);

	FDataLayerActorTreeItem(const FDataLayerActorTreeItemData& InData)
		: FActorTreeItem(InData.Actor.Get())
		, DataLayerInstance(InData.DataLayerInstance)
		, IDDataLayerActor(FDataLayerActorTreeItem::ComputeTreeItemID(Actor.Get(), DataLayerInstance.Get()))
	{
		UpdateDisplayStringInternal();
	}

	UDataLayerInstance* GetDataLayer() const { return DataLayerInstance.Get(); }
	
	const AActor* GetActor() const { return Actor.Get(); }
	AActor* GetActor() { return Actor.Get(); }

	static FSceneOutlinerTreeItemID ComputeTreeItemID(AActor* InActor, const UDataLayerInstance* InDataLayerInstance)
	{
		FObjectKey ActorKey(InActor);
		FObjectKey DataLayerInstanceKey(InDataLayerInstance);

		FArchiveMD5 Ar;
		Ar << ActorKey << DataLayerInstanceKey;

		return FSceneOutlinerTreeItemID(Ar.GetGuidFromHash());
	}

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(Actor.Get(), DataLayerInstance.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(Actor.Get(), DataLayerInstance.Get());
	}

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Actor.IsValid() && DataLayerInstance.IsValid(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return IDDataLayerActor; }
	virtual bool ShouldShowVisibilityState() const { return false; }
	virtual bool HasVisibilityInfo() const override { return false; }
	virtual void OnVisibilityChanged(const bool bNewVisibility) override {}
	virtual bool GetVisibility() const override { return false; }
	/* End ISceneOutlinerTreeItem Implementation */

protected:

	virtual void UpdateDisplayString() override
	{
		UpdateDisplayStringInternal();
	}

private:

	void UpdateDisplayStringInternal()
	{
		FActorTreeItem::UpdateDisplayString();

		if (UWorld* OwningWorld = Actor.IsValid() ? Actor->GetWorld() : nullptr)
		{
			ULevel* Level = Actor.IsValid() ? Actor->GetLevel() : nullptr;
			ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld);
			if (LevelInstanceSubsystem && Level && (Level != OwningWorld->GetCurrentLevel()))
			{
				DisplayString = LevelInstanceSubsystem->PrefixWithParentLevelInstanceActorLabels(DisplayString, Level);
			}
		}
	}

	TWeakObjectPtr<UDataLayerInstance> DataLayerInstance;
	const FSceneOutlinerTreeItemID IDDataLayerActor;
};