// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/DataLayerDragDropOp.h"

#include "Algo/Transform.h"
#include "ClassIconFinder.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Templates/Tuple.h"

class UClass;
struct FSlateBrush;

void FDataLayerDragDropOp::Construct()
{
	const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("DataLayer.Editor"));
	if (DataLayerInstances.Num() == 1 && DataLayerInstances[0].IsValid())
	{
		SetToolTip(FText::FromString(DataLayerInstances[0]->GetDataLayerShortName()), Icon);
	}
	else
	{
		FText Text = FText::Format(NSLOCTEXT("FDataLayerDragDropOp", "MultipleFormat", "{0} DataLayers"), DataLayerInstances.Num());
		SetToolTip(Text, Icon);
	}

	SetupDefaults();
	FDecoratedDragDropOp::Construct();
}

void FDataLayerActorMoveOp::Construct()
{
	// Set text and icon
	TArray< TWeakObjectPtr< AActor > > Actors;
	Algo::TransformIf(DataLayerActorMoveElements, Actors, [](const FDataLayerActorMoveElement& MoveElement) { return MoveElement.Key.IsValid(); }, [](const FDataLayerActorMoveElement& MoveElement) { return MoveElement.Key; });

	UClass* CommonSelClass = nullptr;
	const FSlateBrush* Icon = FClassIconFinder::FindIconForActors(Actors, CommonSelClass);

	if (DataLayerActorMoveElements.Num() == 1)
	{
		SetToolTip(FText::FromString(DataLayerActorMoveElements[0].Key->GetActorLabel()), Icon);
	}
	else
	{
		const FText Text = FText::Format(NSLOCTEXT("FDataLayerActorDragDropOp", "FormatActors", "{0} Actors"), DataLayerActorMoveElements.Num());
		SetToolTip(Text, Icon);
	}

	SetupDefaults();
	FDecoratedDragDropOp::Construct();
}