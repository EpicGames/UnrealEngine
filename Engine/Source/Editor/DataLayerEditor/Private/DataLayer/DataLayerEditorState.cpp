// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/DataLayerEditorState.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "Algo/Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerEditorState)

#define LOCTEXT_NAMESPACE "DataLayersEditorState"

UDataLayerEditorState::UDataLayerEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UDataLayerEditorState::GetCategoryText() const
{
	return FText(LOCTEXT("DataLayersEditorStateCategoryText", "Data Layers"));
}

UEditorState::FOperationResult UDataLayerEditorState::CaptureState()
{
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetStateWorld());
	if (!DataLayerManager)
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("CaptureStateSkipped_NoDataLayerManager", "No data layer manager, world is probably not partitioned"));
	}

	DataLayerManager->ForEachDataLayerInstance([this](const UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance && DataLayerInstance->GetAsset())
		{
			if (DataLayerInstance->IsLoadedInEditor() != DataLayerInstance->IsInitiallyLoadedInEditor())
			{
				if (DataLayerInstance->IsLoadedInEditor())
				{
					LoadedDataLayers.Add(DataLayerInstance->GetAsset());
				}
				else
				{
					NotLoadedDataLayers.Add(DataLayerInstance->GetAsset());
				}
			}
		}

		return true;
	});

	FOperationResult::EResult OperationResult = LoadedDataLayers.IsEmpty() && NotLoadedDataLayers.IsEmpty() ? FOperationResult::Skipped : FOperationResult::Success;
	return FOperationResult(OperationResult, FText::Format(LOCTEXT("CaptureStateSuccess", "LoadedDataLayers={0}, NotLoadedDataLayers={1}"), LoadedDataLayers.Num(), NotLoadedDataLayers.Num()));
}

UEditorState::FOperationResult UDataLayerEditorState::RestoreState() const
{
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetStateWorld());
	if (!DataLayerManager)
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_NoDataLayerManager", "No data layer manager, world is probably not partitioned"));
	}

	// Gather overriden state for all DataLayer instance
	TMap<const UDataLayerInstance*, bool> DataLayersLoadedInEditor;

	// Unloaded
	for (const TObjectPtr<const UDataLayerAsset>& DataLayerAsset : NotLoadedDataLayers)
	{
		if (DataLayerAsset != nullptr)
		{
			const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerAsset);
			if (DataLayerInstance != nullptr)
			{
				DataLayersLoadedInEditor.Emplace(DataLayerInstance, false);
			}
		}
	}

	// Loaded
	for (const TObjectPtr<const UDataLayerAsset>& DataLayerAsset : LoadedDataLayers)
	{
		if (DataLayerAsset != nullptr)
		{
			const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerAsset);
			if (DataLayerInstance != nullptr)
			{
				DataLayersLoadedInEditor.Emplace(DataLayerInstance, true);
			}
		}
	}

	TArray<UDataLayerInstance*> DataLayersLoaded;
	TArray<UDataLayerInstance*> DataLayersUnloaded;
	DataLayerManager->ForEachDataLayerInstance([&DataLayersLoaded, &DataLayersUnloaded, &DataLayersLoadedInEditor](UDataLayerInstance* DataLayerInstance)
	{
		bool bDataLayerLoadedInEditor = DataLayerInstance->IsInitiallyLoadedInEditor();
		if (bool* bBookmarkValue = DataLayersLoadedInEditor.Find(DataLayerInstance))
		{
			bDataLayerLoadedInEditor = *bBookmarkValue;
		}

		TArray<UDataLayerInstance*>& DataLayersArray = bDataLayerLoadedInEditor ? DataLayersLoaded : DataLayersUnloaded;
		DataLayersArray.Add(DataLayerInstance);

		// Visibility of Data Layers is not currently saved to users settings or world bookmarks
		// Until then, we restore them to their initial visibility
		DataLayerInstance->SetVisible(DataLayerInstance->IsInitiallyVisible());

		return true;
	});

	if (UDataLayerEditorSubsystem* DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get())
	{
		const bool LOAD_DATALAYER = true;
		const bool UNLOAD_DATALAYER = false;
		const bool FROM_USER_CHANGE = true;
		DataLayerEditorSubsystem->SetDataLayersIsLoadedInEditor(DataLayersLoaded, LOAD_DATALAYER, FROM_USER_CHANGE);
		DataLayerEditorSubsystem->SetDataLayersIsLoadedInEditor(DataLayersUnloaded, UNLOAD_DATALAYER, FROM_USER_CHANGE);
	}
	
	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("RestoreStateSuccess", "LoadedDataLayers={0}, NotLoadedDataLayers={1}"), LoadedDataLayers.Num(), NotLoadedDataLayers.Num()));
}

#undef LOCTEXT_NAMESPACE
