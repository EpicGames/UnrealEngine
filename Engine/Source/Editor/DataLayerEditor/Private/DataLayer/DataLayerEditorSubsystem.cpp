// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayer/DataLayerAction.h"
#include "DataLayer/DataLayerEditorState.h"
#include "Containers/EnumAsByte.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorState/EditorStateSubsystem.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "LevelEditorViewport.h"
#include "LevelEditorDragDropHandler.h"
#include "ObjectTools.h"
#include "GameFramework/Actor.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/IFilter.h"
#include "Misc/Optional.h"
#include "Misc/ScopedSlowTask.h"
#include "Algo/Transform.h"
#include "Algo/RemoveIf.h"
#include "Algo/AnyOf.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "Subsystems/SubsystemCollection.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Docking/TabManager.h"
#include "DataLayerEditorModule.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerInstancePrivate.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "ActorPartition/PartitionActor.h"
#include "ActorPartition/ActorPartitionSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerEditorSubsystem)

class SWidget;

#define LOCTEXT_NAMESPACE "DataLayer"

DEFINE_LOG_CATEGORY_STATIC(LogDataLayerEditorSubsystem, All, All);

static FName NAME_CurrentDataLayerColor(TEXT("CurrentDataLayerColor"));
static FName NAME_RuntimeDataLayerColor(TEXT("RuntimeDataLayerColor"));
static FName NAME_ExternalDataLayerColor(TEXT("ExternalDataLayerColor"));

FDataLayerCreationParameters::FDataLayerCreationParameters()
	: DataLayerAsset(nullptr)
	, WorldDataLayers(nullptr)
	, bIsPrivate(false)
{}

//////////////////////////////////////////////////////////////////////////
// FDataLayersBroadcast

class FDataLayersBroadcast
{
public:
	FDataLayersBroadcast(UDataLayerEditorSubsystem* InDataLayerEditorSubsystem);
	~FDataLayersBroadcast();
	void Deinitialize();

private:
	void Initialize();
	void OnEditorMapChange(uint32 MapChangeFlags = 0) { DataLayerEditorSubsystem->EditorMapChange(); }
	void OnPostUndoRedo() { DataLayerEditorSubsystem->PostUndoRedo(); }
	void OnNewActorsPlaced(UObject* ObjToUse, const TArray<AActor*>& PlacedActors) { DataLayerEditorSubsystem->OnNewActorsPlaced(ObjToUse, PlacedActors); }
	void OnEditorActorReplaced(AActor* OldActor, AActor* NewActor) {  DataLayerEditorSubsystem->OnEditorActorReplaced(OldActor, NewActor); }
	void OnCurrentLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld) { DataLayerEditorSubsystem->EditorRefreshDataLayerBrowser(); }
	void OnPostWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS) { if (World == DataLayerEditorSubsystem->GetWorld()) { DataLayerEditorSubsystem->EditorMapChange(); } }
	void OnObjectPostEditChange(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	void OnLevelActorsAdded(AActor* InActor) { DataLayerEditorSubsystem->InitializeNewActorDataLayers(InActor); }
	void OnLevelSelectionChanged(UObject* InObject) { DataLayerEditorSubsystem->OnSelectionChanged(); }
	void OnExternalDataLayerAssetRegistrationStateChanged(const UExternalDataLayerAsset* ExternalDataLayerAsset, EExternalDataLayerRegistrationState OldState, EExternalDataLayerRegistrationState NewState) { DataLayerEditorSubsystem->OnExternalDataLayerAssetRegistrationStateChanged(ExternalDataLayerAsset, OldState, NewState); }
	TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> OnLevelEditorDragDropWorldSurrogateReferencingObject(UWorld* ReferencingWorld, const FSoftObjectPath& Object) { return DataLayerEditorSubsystem->OnLevelEditorDragDropWorldSurrogateReferencingObject(ReferencingWorld, Object); }

	UDataLayerEditorSubsystem* DataLayerEditorSubsystem;
	bool bIsInitialized;
};

FDataLayersBroadcast::FDataLayersBroadcast(UDataLayerEditorSubsystem* InDataLayerEditorSubsystem)
	: DataLayerEditorSubsystem(InDataLayerEditorSubsystem)
	, bIsInitialized(false)
{
	Initialize();
}

FDataLayersBroadcast::~FDataLayersBroadcast()
{
	Deinitialize();
}

void FDataLayersBroadcast::Deinitialize()
{
	if (bIsInitialized)
	{
		bIsInitialized = false;

		if (!IsEngineExitRequested())
		{
			FEditorDelegates::MapChange.RemoveAll(this);
			FEditorDelegates::PostUndoRedo.RemoveAll(this);
			FEditorDelegates::OnNewActorsPlaced.RemoveAll(this);
			FEditorDelegates::OnEditorActorReplaced.RemoveAll(this);
			FWorldDelegates::OnCurrentLevelChanged.RemoveAll(this);
			FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);
			FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
			if (GEngine)
			{
				GEngine->OnLevelActorAdded().RemoveAll(this);
			}
			USelection::SelectionChangedEvent.RemoveAll(this);
			USelection::SelectObjectEvent.RemoveAll(this);
			UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
			ExternalDataLayerEngineSubsystem.OnExternalDataLayerAssetRegistrationStateChanged.RemoveAll(this);
			if (ULevelEditorDragDropHandler* DragDrop = GEditor ? GEditor->GetLevelEditorDragDropHandler() : nullptr)
			{
				DragDrop->OnLevelEditorDragDropWorldSurrogateReferencingObject().Unbind();
			}
		}
	}
}

void FDataLayersBroadcast::Initialize()
{
	if (!bIsInitialized)
	{
		bIsInitialized = true;
		FEditorDelegates::MapChange.AddRaw(this, &FDataLayersBroadcast::OnEditorMapChange);
		FEditorDelegates::PostUndoRedo.AddRaw(this, &FDataLayersBroadcast::OnPostUndoRedo);
		FEditorDelegates::OnNewActorsPlaced.AddRaw(this, &FDataLayersBroadcast::OnNewActorsPlaced);
		FEditorDelegates::OnEditorActorReplaced.AddRaw(this, &FDataLayersBroadcast::OnEditorActorReplaced);
		FWorldDelegates::OnCurrentLevelChanged.AddRaw(this, &FDataLayersBroadcast::OnCurrentLevelChanged);
		FWorldDelegates::OnPostWorldInitialization.AddRaw(this, &FDataLayersBroadcast::OnPostWorldInitialization);
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDataLayersBroadcast::OnObjectPostEditChange);
		GEngine->OnLevelActorAdded().AddRaw(this, &FDataLayersBroadcast::OnLevelActorsAdded);
		USelection::SelectionChangedEvent.AddRaw(this, &FDataLayersBroadcast::OnLevelSelectionChanged);
		USelection::SelectObjectEvent.AddRaw(this, &FDataLayersBroadcast::OnLevelSelectionChanged);
		UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
		ExternalDataLayerEngineSubsystem.OnExternalDataLayerAssetRegistrationStateChanged.AddRaw(this, &FDataLayersBroadcast::OnExternalDataLayerAssetRegistrationStateChanged);

		if (ULevelEditorDragDropHandler* DragDrop = GEditor->GetLevelEditorDragDropHandler())
		{
			DragDrop->OnLevelEditorDragDropWorldSurrogateReferencingObject().BindRaw(this, &FDataLayersBroadcast::OnLevelEditorDragDropWorldSurrogateReferencingObject);
		}

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
		// Colorize actor using its Data Layer Debug Color only if the Data Layer is in the Actor Editor Context
		// - For multiple values, use white
		// - Else, use gray
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(NAME_CurrentDataLayerColor, LOCTEXT("CurrentDataLayerColor", "Current Data Layer Color"), [](const UPrimitiveComponent* InPrimitiveComponent) -> FLinearColor
		{
			if (AActor* Actor = InPrimitiveComponent->GetOwner())
			{
				for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstances())
				{
					if (DataLayerInstance->IsActorEditorContextCurrentColorized())
					{
						return DataLayerInstance->GetDebugColor();
					}
				}

				for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstances())
				{
					if (DataLayerInstance->IsInActorEditorContext())
					{
						return FLinearColor::White;
					}
				}
			}
			return FLinearColor::Gray;
		},
		[]() {},
		LOCTEXT("CurrentDataLayerColor_ToolTip", "Colorize actor using its Data Layer Debug Color only if the Data Layer is in the Actor Editor Context. White means multiple values, the rest is Gray.") );

		auto GetActorExternalDataLayerInstance = [](AActor* InActor) -> const UDataLayerInstance*
		{
			if (const UExternalDataLayerAsset* ExternalDataLayerAsset = InActor ? InActor->GetExternalDataLayerAsset() : nullptr)
			{
				if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InActor))
				{
					if (const UDataLayerInstance* ExternalDataLayerInstance = DataLayerManager->GetDataLayerInstance(ExternalDataLayerAsset))
					{
						return ExternalDataLayerInstance;
					}
				}
			}
			return nullptr;
		};

		// Colorize actor using its Runtime Data Layer Debug Color
		// - If 2 Runtime Data Layers and one of them is the EDL, favor the other one
		// - Other cases of multiple Runtime Data Layers, use white
		// - Else, use gray
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(NAME_RuntimeDataLayerColor, LOCTEXT("RuntimeDataLayerColor", "Runtime Data Layer Color"), [](const UPrimitiveComponent* InPrimitiveComponent) -> FLinearColor
		{
			if (AActor* Actor = InPrimitiveComponent->GetOwner())
			{
				TArray<const UDataLayerInstance*> RuntimeDataLayerInstances;
				Algo::TransformIf(Actor->GetDataLayerInstances(), RuntimeDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsRuntime(); }, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance; });
				if (uint32 Count = RuntimeDataLayerInstances.Num())
				{
					if (Count == 1)
					{
						return RuntimeDataLayerInstances[0]->GetDebugColor();
					}
					else if (Count == 2)
					{
						if (RuntimeDataLayerInstances[0]->IsA<UExternalDataLayerInstance>())
						{
							return RuntimeDataLayerInstances[1]->GetDebugColor();
						}
						else if (RuntimeDataLayerInstances[1]->IsA<UExternalDataLayerInstance>())
						{
							return RuntimeDataLayerInstances[0]->GetDebugColor();
						}
					}
					return FLinearColor::White;
				}
			}
			return FLinearColor::Gray;
		},
		[]() {},
		LOCTEXT("RuntimeDataLayerColor_ToolTip", "Colorize actor using its Data Layer Debug Color only if the Data Layer is in the Actor Editor Context. White means multiple values, the rest is Gray."));

		// Colorize actor using its External Data Layer Debug Color (Use gray if none)
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(NAME_ExternalDataLayerColor, LOCTEXT("ExternalDataLayerColor", "External Data Layer Color"), [&GetActorExternalDataLayerInstance](const UPrimitiveComponent* InPrimitiveComponent) -> FLinearColor
		{
			const UDataLayerInstance* ExternalDataLayerInstance = GetActorExternalDataLayerInstance(InPrimitiveComponent->GetOwner());
			return ExternalDataLayerInstance ? ExternalDataLayerInstance->GetDebugColor() : FLinearColor::Gray;
		},
		[]() {},
		LOCTEXT("ExternalDataLayerColor_ToolTip", "Colorize actor using its External Data Layer Debug Color (Use Gray if none)."));
#endif
	}
}

void FDataLayersBroadcast::OnObjectPostEditChange(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!Object || Object->IsTemplate() || PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}
	
	// Ignore changed on game world objects
	const UWorld* World = Object->GetWorld();
	const bool bIsGameWorld = World && World->IsGameWorld();
	if (!bIsGameWorld)
	{
		bool bRefresh = false;
		if (Object->IsA<UDataLayerInstance>() || Object->IsA<UDataLayerAsset>())
		{
			bRefresh = true;
		}
		else if (const AActor* Actor = Cast<AActor>(Object))
		{
			bRefresh = Actor->IsPropertyChangedAffectingDataLayers(PropertyChangedEvent);
		}
		if (bRefresh)
		{
			// Force and update
			DataLayerEditorSubsystem->EditorRefreshDataLayerBrowser();
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// UDataLayerEditorSubsystem
//
// Note: 
//		- DataLayer visibility currently re-uses Actor's bHiddenEdLayer. It's viable since Layer & DataLayer are mutually exclusive systems.
//		- UDataLayerEditorSubsystem is intended to replace ULayersSubsystem for worlds using the World Partition system.
//		  Extra work is necessary to replace all references to GetEditorSubsystem<ULayersSubsystem> in the Editor.
//		  Either a proxy that redirects calls to the proper EditorSubsystem will be used or user code will change to trigger delegate broadcast instead of directly accessing the subsystem (see calls to InitializeNewActorDataLayers everywhere as an example).
//

UDataLayerEditorSubsystem::UDataLayerEditorSubsystem()
: bRebuildSelectedDataLayersFromEditorSelection(false)
, bAsyncBroadcastDataLayerChanged(false)
, bAsyncUpdateAllActorsVisibility(false)
, bAsyncInvalidateViewports(false)
{}

UDataLayerEditorSubsystem* UDataLayerEditorSubsystem::Get()
{
	return GEditor ? GEditor->GetEditorSubsystem<UDataLayerEditorSubsystem>() : nullptr;
}

void UDataLayerEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency<UActorEditorContextSubsystem>();

	Super::Initialize(Collection);

	// Set up the broadcast functions for DataLayerEditorSubsystem
	DataLayersBroadcast = MakeShareable(new FDataLayersBroadcast(this));

	UpdateRegisteredWorldDelegates();

	UActorEditorContextSubsystem::Get()->RegisterClient(this);

	// Register the engine broadcast bridge
	OnActorDataLayersEditorLoadingStateChangedEngineBridgeHandle = DataLayerEditorLoadingStateChanged.AddStatic(&FDataLayersEditorBroadcast::StaticOnActorDataLayersEditorLoadingStateChanged);

	class FDataLayerActorDescFilter : public IWorldPartitionActorLoaderInterface::FActorDescFilter
	{
	public:
		FDataLayerActorDescFilter(UDataLayerEditorSubsystem* InSubsystem) : Subsystem(InSubsystem) {}
		bool PassFilter(class UWorld* InWorld, const FWorldPartitionHandle& InHandle) override
		{
			if (!Subsystem->PassDataLayersFilter(InWorld, InHandle))
			{
				return false;
			}

			return true;
		}

		// Leave [0,9] for Game code
		virtual uint32 GetFilterPriority() const { return 10; }

		virtual FText* GetFilterReason() const override
		{
			static FText UnloadedReason(LOCTEXT("DataLayerFilterReason", "Unloaded Datalayer"));
			return &UnloadedReason;
		}
	private:
		UDataLayerEditorSubsystem* Subsystem;
	};

	// Register actor descriptor loading filter
	IWorldPartitionActorLoaderInterface::RegisterActorDescFilter(MakeShareable<IWorldPartitionActorLoaderInterface::FActorDescFilter>(new FDataLayerActorDescFilter(this)));

	Collection.InitializeDependency<UEditorStateSubsystem>();
	UEditorStateSubsystem::RegisterEditorStateType<UDataLayerEditorState>();
}

void UDataLayerEditorSubsystem::Deinitialize()
{
	UActorEditorContextSubsystem::Get()->UnregisterClient(this);

	Super::Deinitialize();

	DataLayersBroadcast->Deinitialize();

	// Unregister the engine broadcast bridge
	DataLayerEditorLoadingStateChanged.Remove(OnActorDataLayersEditorLoadingStateChangedEngineBridgeHandle);

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_RuntimeDataLayerColor);
	FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_CurrentDataLayerColor);
	FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_ExternalDataLayerColor);
#endif

	UEditorStateSubsystem::UnregisterEditorStateType<UDataLayerEditorState>();
}

bool UDataLayerEditorSubsystem::ShouldHandleActor(AActor* InActor) const
{
	if (GIsReinstancing || InActor->bIsEditorPreviewActor || !InActor->IsPackageExternal() || !InActor->IsMainPackageActor() || !InActor->GetWorld() || (InActor->GetWorld() != GetWorld()))
	{
		return false;
	}
	return true;
}

const UExternalDataLayerInstance* UDataLayerEditorSubsystem::GetActorSpawningExternalDataLayerInstance(AActor* InActor) const
{
	if (!ShouldHandleActor(InActor))
	{
		return nullptr;
	}

	// For backward compatibility, don't resolve an External Data Layer when there's a Content Bundle in the Actor Editor Context
	if (IWorldPartitionEditorModule::Get().IsEditingContentBundle())
	{
		return nullptr;
	}

	const UWorld* OwningWorld = InActor->GetWorld();
	const UObject* OverrideSpawningLevelMountPointObject = ULevel::GetOverrideSpawningLevelMountPointObject();
	const UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(OwningWorld);

	// When the object shares a mount point with the owning world, the actor's
	// default placement (the map package) is already in the right plugin and we shouldn't
	// auto-promote it into an EDL.
	const UPackage* WorldPackage = OwningWorld ? OwningWorld->GetPackage() : nullptr;
	auto IsInDifferentPluginThanWorld = [WorldPackage](const UObject* Object) -> bool
	{
		const UPackage* ObjectPackage = Object ? Object->GetPackage() : nullptr;
		return ObjectPackage && WorldPackage
			&& FPackageName::GetPackageMountPoint(ObjectPackage->GetName()) != FPackageName::GetPackageMountPoint(WorldPackage->GetName());
	};
	
	// Try to get the external data layer for from the override spawning object
	const UExternalDataLayerAsset* ResolvedExternalDataLayerAsset = FExternalDataLayerHelper::GetExternalDataLayerAssetFromObject(OverrideSpawningLevelMountPointObject);
	if (!ResolvedExternalDataLayerAsset && IsInDifferentPluginThanWorld(OverrideSpawningLevelMountPointObject))
	{
		// If none found, try matching an external data layer with the override spawning object
		ResolvedExternalDataLayerAsset = ExternalDataLayerManager ? ExternalDataLayerManager->GetMatchingExternalDataLayerAssetForObjectPath(OverrideSpawningLevelMountPointObject) : nullptr;
	}
	if (!ResolvedExternalDataLayerAsset && IsInDifferentPluginThanWorld(InActor->GetClass()))
	{
		// If none found, try matching an external data layer with the actor class
		ResolvedExternalDataLayerAsset = ExternalDataLayerManager ? ExternalDataLayerManager->GetMatchingExternalDataLayerAssetForObjectPath(InActor->GetClass()) : nullptr;
	}
	if (!ResolvedExternalDataLayerAsset)
	{
		// Fallback on actor editor context external data layer
		ResolvedExternalDataLayerAsset = GetActorEditorContextCurrentExternalDataLayer();
	}

	const UExternalDataLayerInstance* ResolvedExternalDataLayerInstance = (ResolvedExternalDataLayerAsset && ExternalDataLayerManager) ? ExternalDataLayerManager->GetExternalDataLayerInstance(ResolvedExternalDataLayerAsset) : nullptr;
	return ResolvedExternalDataLayerInstance;
}

void UDataLayerEditorSubsystem::MoveActorToExternalDataLayer(AActor* InActor, const UExternalDataLayerInstance* InExternalDataLayerInstance, bool bInNotifyFailureReason)
{
	FText FailureReason;
	if (!FExternalDataLayerHelper::MoveActorsToExternalDataLayer({ InActor }, InExternalDataLayerInstance, &FailureReason))
	{
		if (bInNotifyFailureReason)
		{
			UE_LOGF(LogWorldPartition, Warning, "%ls", *FailureReason.ToString());
			LastWarningNotification = FailureReason;
		}
	}
};

void UDataLayerEditorSubsystem::OnActorPreSpawnInitialization(AActor* InActor)
{
	if (const UExternalDataLayerInstance* ExternalDataLayerInstance = GetActorSpawningExternalDataLayerInstance(InActor))
	{
		check(!InActor->GetExternalDataLayerAsset());
		MoveActorToExternalDataLayer(InActor, ExternalDataLayerInstance);
	}
}

void UDataLayerEditorSubsystem::OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor)
{
	// Try to apply the current context on the new replacing actor
	const bool bForceTryApply = true;
	ApplyContext(InNewActor, bForceTryApply, InOldActor);

	if (ShouldHandleActor(InNewActor))
	{
		// Here we repair the case where the actor package doesn't match the actor EDL asset.
		// This can happen when using 'replace actor' as it reuses the old actor's package.
		// In this case, choose the EDL from the package if any.
		UPackage* ActorPackage = InNewActor->GetExternalPackage();
		const UWorld* OwningWorld = InNewActor->GetWorld();
		if (const UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(OwningWorld); ActorPackage && ExternalDataLayerManager)
		{
			// Only move actors which are actually stored in an EDL. Otherwise, if the owning world
			// shared a mount point with an EDL, the actor would erroneously be moved into that EDL.
			if (FExternalDataLayerHelper::IsExternalDataLayerPath(ActorPackage->GetPathName(), nullptr))
			{
				const UExternalDataLayerAsset* PackageDataLayerAsset = ExternalDataLayerManager->GetMatchingExternalDataLayerAssetForObjectPath(ActorPackage->GetPathName());
				if (PackageDataLayerAsset != InNewActor->GetExternalDataLayerAsset())
				{
					const UExternalDataLayerInstance* ExternalDataLayerInstance = ExternalDataLayerManager->GetExternalDataLayerInstance(PackageDataLayerAsset);
					MoveActorToExternalDataLayer(InNewActor, ExternalDataLayerInstance);
				}
			}
		}
	}
}

void UDataLayerEditorSubsystem::OnNewActorsPlaced(UObject* InObjToUse, const TArray<AActor*>& InPlacedActors)
{
	for (AActor* PlacedActor : InPlacedActors)
	{
		// Try to apply the current context after actor is placed
		const bool bForceTryApply = true;
		ApplyContext(PlacedActor, bForceTryApply);
	}
}

UWorld* UDataLayerEditorSubsystem::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

ETickableTickType UDataLayerEditorSubsystem::GetTickableTickType() const
{
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Conditional;
}

bool UDataLayerEditorSubsystem::IsTickable() const
{
	return GetWorld() && (bAsyncBroadcastDataLayerChanged || bAsyncUpdateAllActorsVisibility || bAsyncInvalidateViewports || LastWarningNotification.IsSet());
}

void UDataLayerEditorSubsystem::Tick(float DeltaTime)
{
	if (bAsyncBroadcastDataLayerChanged)
	{
		BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
		bAsyncBroadcastDataLayerChanged = false;
	}

	if (bAsyncUpdateAllActorsVisibility)
	{
		UpdateAllActorsVisibility(false, false);
		bAsyncUpdateAllActorsVisibility = false;
	}

	if (bAsyncInvalidateViewports)
	{
		GEditor->RedrawLevelEditingViewports();
		bAsyncInvalidateViewports = false;
	}

	if (LastWarningNotification.IsSet())
	{
		// Trigger a notification with the last pushed warning (avoids spamming notification manager)
		FNotificationInfo WarningInfo(LastWarningNotification.GetValue());
		WarningInfo.ExpireDuration = 3.0f;
		WarningInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
		WarningInfo.HyperlinkText = LOCTEXT("ShowMessageLogHyperlink", "Show Output Log");
		FSlateNotificationManager::Get().AddNotification(WarningInfo);
		LastWarningNotification.Reset();
	}
}

void UDataLayerEditorSubsystem::BeginDestroy()
{
	if (DataLayersBroadcast)
	{
		DataLayersBroadcast->Deinitialize();
		DataLayersBroadcast.Reset();
	}

	Super::BeginDestroy();
}

void UDataLayerEditorSubsystem::ApplyContext(AActor* InActor, bool bInForceTryApply, AActor* InReplacedActor)
{
	if (!ShouldHandleActor(InActor))
	{
		return;
	}

	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InActor->GetWorld());
	if (!DataLayerManager)
	{
		return;
	}

	// Try to apply context External Data Layer (this operation can fail if asset referencing validation fails)
	const UExternalDataLayerAsset* CurrentExternalDataLayer = GetActorEditorContextCurrentExternalDataLayer();
	const UExternalDataLayerAsset* ActorExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
	const UExternalDataLayerAsset* ReplacedActorExternalDataLayerAsset = InReplacedActor ? InReplacedActor->GetExternalDataLayerAsset() : nullptr;
	if (CurrentExternalDataLayer)
	{
		if (ActorExternalDataLayerAsset != CurrentExternalDataLayer)
		{
			// Don't apply if there's a valid override spawning External Data Layer (see OnActorPreSpawnInitialization)
			const UExternalDataLayerInstance* SpawningExternalDataLayer = ULevel::GetOverrideSpawningLevelMountPointObject() ? GetActorSpawningExternalDataLayerInstance(InActor) : nullptr;
			if (!SpawningExternalDataLayer || bInForceTryApply)
			{
				UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(InActor);
				if (const UExternalDataLayerInstance* CurrentExternalDataLayerInstance = CurrentExternalDataLayer ? ExternalDataLayerManager->GetExternalDataLayerInstance(CurrentExternalDataLayer) : nullptr)
				{
					MoveActorToExternalDataLayer(InActor, CurrentExternalDataLayerInstance);
				}
			}
		}
	}
	// When replacing an actor, try to match the replaced actor External Data Layer if there's none set in the Actor Editor Context
	else if (ReplacedActorExternalDataLayerAsset && (ActorExternalDataLayerAsset != ReplacedActorExternalDataLayerAsset))
	{
		UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(InActor);
		if (const UExternalDataLayerInstance* ReplacedExternalDataLayerInstance = ExternalDataLayerManager ? ExternalDataLayerManager->GetExternalDataLayerInstance(ReplacedActorExternalDataLayerAsset) : nullptr)
		{
			// As this is an attempt, no need to report failures
			const bool bNotifyFailureReason = false;
			MoveActorToExternalDataLayer(InActor, ReplacedExternalDataLayerInstance, bNotifyFailureReason);
		}
	}

	// Apply context Data Layers (except External Data Layer)
	TArray<UDataLayerInstance*> DataLayerInstances = DataLayerManager->GetActorEditorContextDataLayers();
	if (!DataLayerInstances.IsEmpty())
	{
		DataLayerInstances.SetNum(Algo::RemoveIf(DataLayerInstances, [](UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); }));
		AddActorToDataLayers(InActor, DataLayerInstances);
		InActor->FixupDataLayers();
	}
}

void UDataLayerEditorSubsystem::OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor)
{
	UE_CLOGF(!InWorld, LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld))
	{
		switch (InType)
		{
		case EActorEditorContextAction::ApplyContext:
			check(InActor && InActor->GetWorld() == InWorld);
			{
				ApplyContext(InActor);
			}
			break;
		case EActorEditorContextAction::ResetContext:
			for (UDataLayerInstance* DataLayerInstance : DataLayerManager->GetActorEditorContextDataLayers())
			{
				RemoveFromActorEditorContext(DataLayerInstance);
			}
			break;
		case EActorEditorContextAction::PushContext:
		case EActorEditorContextAction::PushDuplicateContext:
			DataLayerManager->PushActorEditorContext(InType == EActorEditorContextAction::PushDuplicateContext);
			BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
			break;
		case EActorEditorContextAction::PopContext:
			DataLayerManager->PopActorEditorContext();
			BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
			break;
		case EActorEditorContextAction::InitializeContextFromActor:
			for (const UDataLayerInstance* DataLayerInstance : InActor->GetDataLayerInstances())
			{
				if (DataLayerInstance->CanBeInActorEditorContext())
				{
					const_cast<UDataLayerInstance*>(DataLayerInstance)->AddToActorEditorContext();
				}
			}
			break;
		}
	}
}

void UDataLayerEditorSubsystem::CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const
{
	UActorEditorContextDataLayerState* State = nullptr;

	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld))
	{
		for (UDataLayerInstance* DataLayerInstance : DataLayerManager->GetActorEditorContextDataLayers())
		{
			if (const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset())
			{
				if (State == nullptr)
				{
					State = NewObject<UActorEditorContextDataLayerState>(InStateCollection);
				}

				if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(DataLayerAsset))
				{
					State->ExternalDataLayerAsset = ExternalDataLayerAsset;
				}
				else
				{
					State->DataLayerAssets.Add(DataLayerAsset);
				}
			}
		}
	}

	if (State)
	{
		InStateCollection->AddState(State);
	}
}

void UDataLayerEditorSubsystem::RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection)
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld))
	{
		TSet<UDataLayerInstance*> CurrentDataLayers(DataLayerManager->GetActorEditorContextDataLayers());
		TSet<const UDataLayerInstance*> DataLayersToRestore;

		if (const UActorEditorContextDataLayerState* State = InStateCollection->GetState<UActorEditorContextDataLayerState>())
		{
			for (const TSoftObjectPtr<const UDataLayerAsset>& DLAsset : State->DataLayerAssets)
			{
				const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAssetName(FName(DLAsset.ToString()));
				if (DataLayerInstance && DataLayerInstance->CanBeInActorEditorContext())
				{
					DataLayersToRestore.Add(DataLayerInstance);
				}
			}
		}

		// Add DL instances not in current context
		for (const UDataLayerInstance* DataLayerInstance : DataLayersToRestore)
		{
			if (!CurrentDataLayers.Contains(DataLayerInstance))
			{
				AddToActorEditorContext(const_cast<UDataLayerInstance*>(DataLayerInstance));
			}
		}

		// Remove DL instances currently in the context but not in the state we are restoring
		for (UDataLayerInstance* DataLayerInstance : CurrentDataLayers)
		{
			if (!DataLayersToRestore.Contains(DataLayerInstance))
			{
				RemoveFromActorEditorContext(DataLayerInstance);
			}
		}
	}
}

bool UDataLayerEditorSubsystem::GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld))
	{
		if (!DataLayerManager->GetActorEditorContextDataLayers().IsEmpty())
		{
			OutDiplayInfo.Title = TEXT("Data Layers");
			OutDiplayInfo.Brush = FAppStyle::GetBrush(TEXT("DataLayer.Editor"));
			return true;
		}
	}
	return false;
}

TSharedRef<SWidget> UDataLayerEditorSubsystem::GetActorEditorContextWidget(UWorld* InWorld) const
{
	TSharedRef<SVerticalBox> OutWidget = SNew(SVerticalBox);

	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld))
	{
		TArray<UDataLayerInstance*> DataLayers = DataLayerManager->GetActorEditorContextDataLayers();
		for (UDataLayerInstance* DataLayerInstance : DataLayers)
		{
			check(IsValid(DataLayerInstance));
			OutWidget->AddSlot().AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 1.0f, 1.0f, 1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(DataLayerInstance->GetDebugColor())
					.Image(FAppStyle::Get().GetBrush("DataLayer.ColorIcon"))
					.DesiredSizeOverride(FVector2D(8, 8))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 1.0f, 1.0f, 1.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(DataLayerInstance->GetDataLayerShortName()))
				]
			];
		}
	}
	
	return OutWidget;
}

void UDataLayerEditorSubsystem::AddToActorEditorContext(UDataLayerInstance* InDataLayerInstance)
{
	check(InDataLayerInstance->CanBeInActorEditorContext());
	if (InDataLayerInstance->AddToActorEditorContext())
	{
		BroadcastDataLayerChanged(EDataLayerAction::Modify, InDataLayerInstance, NAME_None);
		ActorEditorContextClientChanged.Broadcast(this);
	}
}

void UDataLayerEditorSubsystem::RemoveFromActorEditorContext(UDataLayerInstance* InDataLayerInstance)
{
	if (InDataLayerInstance->RemoveFromActorEditorContext())
	{
		BroadcastDataLayerChanged(EDataLayerAction::Modify, InDataLayerInstance, NAME_None);
		ActorEditorContextClientChanged.Broadcast(this);
	}
}

const UExternalDataLayerAsset* UDataLayerEditorSubsystem::GetActorEditorContextCurrentExternalDataLayer() const
{
	UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(GetWorld());
	return ExternalDataLayerManager ? ExternalDataLayerManager->GetActorEditorContextCurrentExternalDataLayer() : nullptr;
}

bool UDataLayerEditorSubsystem::SetActorEditorContextCurrentExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	if (const UExternalDataLayerInstance* ExternalDataLayerInstance = DataLayerManager ? Cast<UExternalDataLayerInstance>(DataLayerManager->GetDataLayerInstance(InExternalDataLayerAsset)) : nullptr)
	{
		if (ExternalDataLayerInstance->CanBeInActorEditorContext())
		{
			AddToActorEditorContext(const_cast<UExternalDataLayerInstance*>(ExternalDataLayerInstance));
			return true;
		}
	}
	else if (const UExternalDataLayerAsset* CurrentExternalDataLayer = GetActorEditorContextCurrentExternalDataLayer())
	{
		if (const UExternalDataLayerInstance* CurrentExternalDataLayerInstance = DataLayerManager ? Cast<UExternalDataLayerInstance>(DataLayerManager->GetDataLayerInstance(CurrentExternalDataLayer)) : nullptr)
		{
			RemoveFromActorEditorContext(const_cast<UExternalDataLayerInstance*>(CurrentExternalDataLayerInstance));
		}
	}
	return false;
}

TArray<const UDataLayerInstance*> UDataLayerEditorSubsystem::GetDataLayerInstances(const TArray<const UDataLayerAsset*> DataLayerAssets) const
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		return DataLayerManager->GetDataLayerInstances(DataLayerAssets);
	}

	return TArray<const UDataLayerInstance*>();
}

void UDataLayerEditorSubsystem::UpdateRegisteredWorldDelegates()
{
	if (UWorld* PreviousWorld = LastRegisteredWorldDelegates.Get())
	{
		PreviousWorld->RemoveOnActorPreSpawnInitialization(OnActorPreSpawnInitializationDelegate);
		PreviousWorld->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
		PreviousWorld->OnWorldPartitionInitialized().RemoveAll(this);
		PreviousWorld->OnWorldPartitionUninitialized().RemoveAll(this);
	}

	LastRegisteredWorldDelegates.Reset();
	OnActorPreSpawnInitializationDelegate.Reset();

	if (UWorld* World = GetWorld())
	{
		LastRegisteredWorldDelegates = World;
		OnActorPreSpawnInitializationDelegate = World->AddOnActorPreSpawnInitialization(FOnActorSpawned::FDelegate::CreateUObject(this, &UDataLayerEditorSubsystem::OnActorPreSpawnInitialization));
		World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddUObject(this, &UDataLayerEditorSubsystem::OnLoadedActorAddedToLevel);
		World->OnWorldPartitionInitialized().AddUObject(this, &UDataLayerEditorSubsystem::OnWorldPartitionInitialized);
		World->OnWorldPartitionUninitialized().AddUObject(this, &UDataLayerEditorSubsystem::OnWorldPartitionUninitialized);
	}
}

void UDataLayerEditorSubsystem::EditorMapChange()
{
	UpdateRegisteredWorldDelegates();
	BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
	UpdateAllActorsVisibility(true, true);
}

void UDataLayerEditorSubsystem::EditorRefreshDataLayerBrowser()
{
	bAsyncBroadcastDataLayerChanged = true;
	bAsyncUpdateAllActorsVisibility = true;
}

void UDataLayerEditorSubsystem::PostUndoRedo()
{
	BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
	UpdateAllActorsVisibility(true, true);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on an individual actor.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool UDataLayerEditorSubsystem::IsActorValidForDataLayerInstances(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayerInstances)
{
	TSet<TSubclassOf<UDataLayerInstance>> DataLayerInstanceClasses;
	for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
	{
		DataLayerInstanceClasses.Add(DataLayerInstance->GetClass());
	}
	return IsActorValidForDataLayerForClasses(Actor, DataLayerInstanceClasses);
}

bool UDataLayerEditorSubsystem::IsActorValidForDataLayerForClasses(AActor* Actor, const TSet<TSubclassOf<UDataLayerInstance>>& DataLayerInstanceClasses)
{
	UWorld* World = Actor ? Actor->GetWorld() : nullptr;
	if (World && (World->WorldType == EWorldType::Editor) && World->IsPartitionedWorld() && Actor && ((Actor->GetLevel() == Actor->GetWorld()->PersistentLevel) || Actor->GetLevel()->GetWorldDataLayers()))
	{
		for (const TSubclassOf<UDataLayerInstance>& DataLayerInstanceClass : DataLayerInstanceClasses)
		{
			if (!Actor->SupportsDataLayerType(DataLayerInstanceClass))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void UDataLayerEditorSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	ULevel* WorldPartitionLevel = InWorldPartition->GetTypedOuter<ULevel>();
	WorldPartitionLevel->OnLoadedActorAddedToLevelEvent.AddUObject(this, &UDataLayerEditorSubsystem::OnLoadedActorAddedToLevel);
	UpdateAllActorsVisibility(true, true, WorldPartitionLevel);
}

void UDataLayerEditorSubsystem::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	InWorldPartition->GetTypedOuter<ULevel>()->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
}

void UDataLayerEditorSubsystem::OnLoadedActorAddedToLevel(AActor& InActor)
{
	InitializeNewActorDataLayers(&InActor);
}

void UDataLayerEditorSubsystem::InitializeNewActorDataLayers(AActor* Actor)
{
	Actor->FixupDataLayers();

	// update general actor visibility
	bool bActorModified = false;
	bool bActorSelectionChanged = false;
	UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, /*bActorNotifySelectionChange*/true, /*bActorRedrawViewports*/false);
	bAsyncInvalidateViewports = true;
}

UWorld* UDataLayerEditorSubsystem::GetWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}

void UDataLayerEditorSubsystem::SetParentDataLayerForDataLayers(const TArray<UDataLayerInstance*>& DataLayers, UDataLayerInstance* ParentDataLayer)
{
	SetParentDataLayerForDataLayersInternal(DataLayers, ParentDataLayer);
}

bool UDataLayerEditorSubsystem::SetParentDataLayerForDataLayersInternal(const TArray<UDataLayerInstance*>& DataLayers, UDataLayerInstance* ParentDataLayer)
{
	bool bResult = false;
	bool bLoadingStateChanged = false;

	for (UDataLayerInstance* DataLayerInstance : DataLayers)
	{
		if (DataLayerInstance->CanBeChildOf(ParentDataLayer))
		{
			const bool bIsLoaded = DataLayerInstance->IsEffectiveLoadedInEditor();
			DataLayerInstance->SetParent(ParentDataLayer);
			if (bIsLoaded != DataLayerInstance->IsEffectiveLoadedInEditor())
			{
				bLoadingStateChanged = true;
			}
			bResult = true;
		}
	}
	if (bResult)
	{
		BroadcastDataLayerChanged(EDataLayerAction::Reset, NULL, NAME_None);
		UpdateAllActorsVisibility(true, true);
		if (bLoadingStateChanged)
		{
			OnDataLayerEditorLoadingStateChanged(true);
		}
	}
	return bResult;
}

bool UDataLayerEditorSubsystem::SetParentDataLayer(UDataLayerInstance* DataLayerInstance, UDataLayerInstance* ParentDataLayer)
{
	return SetParentDataLayerForDataLayersInternal({ DataLayerInstance }, ParentDataLayer);
}

void UDataLayerEditorSubsystem::SetDataLayerInitialRuntimeState(UDataLayerInstance* DataLayerInstance, EDataLayerRuntimeState InitialRuntimeState)
{
	if (DataLayerInstance)
	{
		DataLayerInstance->SetInitialRuntimeState(InitialRuntimeState);
	}
}

void UDataLayerEditorSubsystem::SetDataLayerIsInitiallyVisible(UDataLayerInstance* DataLayerInstance, bool bIsInitiallyVisible)
{
	if (DataLayerInstance)
	{
		DataLayerInstance->SetIsInitiallyVisible(bIsInitiallyVisible);
	}
}

bool UDataLayerEditorSubsystem::AddActorToDataLayer(AActor* Actor, UDataLayerInstance* DataLayerInstance)
{
	TArray<AActor*> Actors;
	Actors.Add(Actor);

	return AddActorsToDataLayers(Actors, { DataLayerInstance });
}

bool UDataLayerEditorSubsystem::AddActorToDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers)
{
	TArray<AActor*> Actors;
	Actors.Add(Actor);

	return AddActorsToDataLayers(Actors, DataLayers);
}

bool UDataLayerEditorSubsystem::AddActorsToDataLayer(const TArray<AActor*>& Actors, UDataLayerInstance* DataLayerInstance)
{
	return AddActorsToDataLayers(Actors, { DataLayerInstance });
}

bool UDataLayerEditorSubsystem::AddActorsToDataLayers(const TArray<AActor*>& Actors, const TArray<UDataLayerInstance*>& DataLayerInstances)
{
	bool bChangesOccurred = false;

	if (DataLayerInstances.Num() > 0)
	{
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();

		for (AActor* Actor : Actors)
		{
			if (!IsActorValidForDataLayerInstances(Actor, DataLayerInstances))
			{
				continue;
			}

			bool bActorWasModified = false;
			for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
			{
				if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
				{
					// If actor's level WorldDataLayers doesn't match this DataLayerInstance outer WorldDataLayers, 
					// Make sure that a DataLayer Instance for this Data Layer Asset exists in the Actor's level WorldDataLayers.
					// Skip this for External Data Layers as they are only applied to the parent LevelInstance actor
					if (!DataLayerInstanceWithAsset->IsA<UExternalDataLayerInstance>())
					{
						AWorldDataLayers* TargetWorldDataLayers = Actor->GetLevel()->GetWorldDataLayers();
						if (TargetWorldDataLayers != DataLayerInstance->GetOuterWorldDataLayers())
						{
							UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(Actor);
							if (ensureMsgf(DataLayerManager, TEXT("No DataLayerManager found for Actor %s, can't add actors to data layers."), *Actor->GetName()))
							{
								DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerInstanceWithAsset->GetAsset());

								bool bDataLayerInstanceExistsInActorLevel = DataLayerInstance != nullptr;
								if (!bDataLayerInstanceExistsInActorLevel)
								{
									DataLayerInstance = CreateDataLayerInstance<UDataLayerInstanceWithAsset>(TargetWorldDataLayers, DataLayerInstanceWithAsset->GetAsset());
								}
							}
						}
					}
				}

				if (DataLayerInstance)
				{
					if (Actor->AddDataLayer(DataLayerInstance))
					{
						bActorWasModified = true;
						BroadcastActorDataLayersChanged(Actor);
					}
				}
			}

			if (bActorWasModified)
			{
				// Update general actor visibility
				bool bActorModified = false;
				bool bActorSelectionChanged = false;
				UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, /*bActorNotifySelectionChange*/true, /*bActorRedrawViewports*/false);

				bChangesOccurred = true;
			}
		}

		GEditor->GetSelectedActors()->EndBatchSelectOperation();
	}

	return bChangesOccurred;
}

bool UDataLayerEditorSubsystem::RemoveActorFromAllDataLayers(AActor* Actor)
{
	return RemoveActorsFromAllDataLayers({ Actor });
}

bool UDataLayerEditorSubsystem::RemoveActorsFromAllDataLayers(const TArray<AActor*>& Actors)
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	bool bRemoveAllDataLayersOnAllActor = true;
	for (AActor* Actor : Actors)
	{
		TArray<const UDataLayerInstance*> RemovedDataLayers = Actor->RemoveAllDataLayers();
		if (!RemovedDataLayers.IsEmpty())
		{
			for (const UDataLayerInstance* DataLayerInstance : RemovedDataLayers)
			{
				BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, NAME_None);
			}
			BroadcastActorDataLayersChanged(Actor);

			// Update general actor visibility
			bool bActorModified = false;
			bool bActorSelectionChanged = false;
			UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, /*bActorNotifySelectionChange*/true, /*bActorRedrawViewports*/false);

			bRemoveAllDataLayersOnAllActor &= !Actor->HasDataLayers();
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	return bRemoveAllDataLayersOnAllActor;
}

bool UDataLayerEditorSubsystem::RemoveActorFromDataLayer(AActor* Actor, UDataLayerInstance* DataLayerInstance)
{
	TArray<AActor*> Actors;
	Actors.Add(Actor);

	return RemoveActorsFromDataLayers(Actors, { DataLayerInstance });
}

bool UDataLayerEditorSubsystem::RemoveActorFromDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers)
{
	TArray<AActor*> Actors;
	Actors.Add(Actor);

	return RemoveActorsFromDataLayers(Actors, DataLayers);
}

bool UDataLayerEditorSubsystem::RemoveActorsFromDataLayer(const TArray<AActor*>& Actors, UDataLayerInstance* DataLayerInstance)
{
	return RemoveActorsFromDataLayers(Actors, { DataLayerInstance });
}

bool UDataLayerEditorSubsystem::RemoveActorsFromDataLayers(const TArray<AActor*>& Actors, const TArray<UDataLayerInstance*>& DataLayerInstances)
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	bool bChangesOccurred = false;
	for (AActor* Actor : Actors)
	{
		if (!IsActorValidForDataLayerInstances(Actor, DataLayerInstances))
		{
			continue;
		}

		bool bActorWasModified = false;
		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			if (Actor->RemoveDataLayer(DataLayerInstance))
			{
				bActorWasModified = true;
				BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, NAME_None);
				BroadcastActorDataLayersChanged(Actor);
			}
		}

		if (bActorWasModified)
		{
			// Update general actor visibility
			bool bActorModified = false;
			bool bActorSelectionChanged = false;
			UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, /*bActorNotifySelectionChange*/true, /*bActorRedrawViewports*/false);

			bChangesOccurred = true;
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	return bChangesOccurred;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on selected actors.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
TArray<AActor*> UDataLayerEditorSubsystem::GetSelectedActors() const
{
	TArray<AActor*> CurrentlySelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(CurrentlySelectedActors);
	return CurrentlySelectedActors;
}

bool UDataLayerEditorSubsystem::AddSelectedActorsToDataLayer(UDataLayerInstance* DataLayerInstance)
{
	return AddActorsToDataLayer(GetSelectedActors(), DataLayerInstance);
}

bool UDataLayerEditorSubsystem::RemoveSelectedActorsFromDataLayer(UDataLayerInstance* DataLayerInstance)
{
	return RemoveActorsFromDataLayer(GetSelectedActors(), DataLayerInstance);
}

bool UDataLayerEditorSubsystem::AddSelectedActorsToDataLayers(const TArray<UDataLayerInstance*>& DataLayers)
{
	return AddActorsToDataLayers(GetSelectedActors(), DataLayers);
}

bool UDataLayerEditorSubsystem::RemoveSelectedActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers)
{
	return RemoveActorsFromDataLayers(GetSelectedActors(), DataLayers);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on actors in DataLayers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool UDataLayerEditorSubsystem::SelectActorsInDataLayer(UDataLayerInstance* DataLayerInstance, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden)
{
	return SelectActorsInDataLayer(DataLayerInstance, bSelect, bNotify, bSelectEvenIfHidden, nullptr);
}

bool UDataLayerEditorSubsystem::SelectActorsInDataLayer(UDataLayerInstance* DataLayerInstance, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr<FActorFilter>& Filter)
{
	return SelectActorsInDataLayers({ DataLayerInstance }, bSelect, bNotify, bSelectEvenIfHidden, Filter);
}

bool UDataLayerEditorSubsystem::SelectActorsInDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden)
{
	return SelectActorsInDataLayers(DataLayerInstances, bSelect, bNotify, bSelectEvenIfHidden, nullptr);
}

bool UDataLayerEditorSubsystem::SelectActorsInDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr<FActorFilter>& Filter)
{
	if (DataLayerInstances.Num() == 0)
	{
		return true;
	}

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bChangesOccurred = false;

	// Iterate over all actors, looking for actors in the specified DataLayers.
	for (AActor* Actor : FActorRange(GetWorld()))
	{
		if (!IsActorValidForDataLayerInstances(Actor, DataLayerInstances))
		{
			continue;
		}

		if (Filter.IsValid() && !Filter->PassesFilter(Actor))
		{
			continue;
		}

		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			if (Actor->ContainsDataLayer(DataLayerInstance) || Actor->GetDataLayerInstancesForLevel().Contains(DataLayerInstance))
			{
				// The actor was found to be in a specified DataLayerInstance. Set selection state and move on to the next actor.
				bool bNotifyForActor = false;
				GEditor->GetSelectedActors()->Modify();
				GEditor->SelectActor(Actor, bSelect, bNotifyForActor, bSelectEvenIfHidden);
				bChangesOccurred = true;
				break;
			}
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	if (bNotify)
	{
		GEditor->NoteSelectionChange();
	}

	return bChangesOccurred;
}

void UDataLayerEditorSubsystem::SetActorsPinStateInDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, const bool bPinned)
{
	UWorld* World = GetWorld();
	if (UWorldPartition* WorldPartition = World ? World->GetWorldPartition() : nullptr)
	{
		TSet<FName> DataLayerInstanceNames;
		Algo::TransformIf(DataLayerInstances, DataLayerInstanceNames, [](UDataLayerInstance* DataLayerInstance) { return !!DataLayerInstance; }, [](UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetFName(); });

		if (DataLayerInstanceNames.Num())
		{
			TArray<FGuid> ActorGuids;
			for (FActorDescContainerInstanceCollection::TIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
			{
				const FDataLayerInstanceNames& ActorDescDataLayerInstanceNames = Iterator->GetDataLayerInstanceNames();
				if (ActorDescDataLayerInstanceNames.Num())
				{
					for (const FName& DataLayerInstance : DataLayerInstanceNames)
					{
						if (ActorDescDataLayerInstanceNames.Contains(DataLayerInstance))
						{
							ActorGuids.Add(Iterator->GetGuid());
						}
					}
				}
			}

			if (ActorGuids.Num())
			{
				if (bPinned)
				{
					WorldPartition->PinActors(ActorGuids);
				}
				else
				{
					WorldPartition->UnpinActors(ActorGuids);
				}
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on actor viewport visibility regarding DataLayers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool UDataLayerEditorSubsystem::UpdateActorVisibility(AActor* Actor, bool& bOutSelectionChanged, bool& bOutActorModified, const bool bNotifySelectionChange, const bool bRedrawViewports)
{
	bOutActorModified = false;
	bOutSelectionChanged = false;

	// If the actor doesn't belong to any DataLayers
	TArray<const UDataLayerInstance*> DataLayerInstances = Actor->GetDataLayerInstances();
	if (DataLayerInstances.IsEmpty())
	{
		// Actors that don't belong to any DataLayerInstance shouldn't be hidden
		bOutActorModified = Actor->SetIsHiddenEdLayer(false);
		return bOutActorModified;
	}

	bool bActorVisible = false;
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	if (const bool bActorShouldBeLoaded = DataLayerManager ? DataLayerManager->ResolveIsLoadedInEditor(Actor->GetDataLayerInstanceNames()) : true)
	{
		const UExternalDataLayerInstance* ExternalDataLayerInstance = nullptr;
		if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Actor->GetExternalDataLayerAsset())
		{
			UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(GetWorld());
			ExternalDataLayerInstance = ExternalDataLayerManager ? ExternalDataLayerManager->GetExternalDataLayerInstance(ExternalDataLayerAsset) : nullptr;
			DataLayerInstances.Remove(ExternalDataLayerInstance);
		}

		// Actor is hidden if its external data layer is not visibile
		const bool bActorHiddenByEDL = ExternalDataLayerInstance && !ExternalDataLayerInstance->IsEffectiveVisible();
		if (!bActorHiddenByEDL)
		{
			// Else, actor is visible if any of its data layer is visible
			bActorVisible = DataLayerInstances.Num() ? Algo::AnyOf(DataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsEffectiveVisible(); }) : true;
		}
	}

	const bool bIsHiddenEdLayer = !bActorVisible;
	if (Actor->SetIsHiddenEdLayer(bIsHiddenEdLayer))
	{
		bOutActorModified = true;
	}
		
	// If the actor is hidden, de-select it.
	if (bIsHiddenEdLayer)
	{
		// If the actor was selected, mark it as unselected
		if (Actor->IsSelected())
		{
			bool bSelect = false;
			bool bNotify = false;
			bool bIncludeHidden = true;
			GEditor->SelectActor(Actor, bSelect, bNotify, bIncludeHidden);

			bOutSelectionChanged = true;
			bOutActorModified = true;
		}
	}

	if (bNotifySelectionChange && bOutSelectionChanged)
	{
		GEditor->NoteSelectionChange();
	}

	if (bRedrawViewports)
	{
		GEditor->RedrawLevelEditingViewports();
	}

	if (bOutActorModified || bOutSelectionChanged)
	{
		bAsyncInvalidateViewports = true;
		return true;
	}
	return false;;
}

bool UDataLayerEditorSubsystem::UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports)
{
	return UpdateAllActorsVisibility(bNotifySelectionChange, bRedrawViewports, nullptr);
}

bool UDataLayerEditorSubsystem::UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports, ULevel* InLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDataLayerEditorSubsystem::UpdateAllActorsVisibility);

	bool bSelectionChanged = false;
	bool bChangesOccurred = false;

	auto UpdateActorVisibilityLambda = [this, &bSelectionChanged, &bChangesOccurred](AActor* Actor)
	{
		if (Actor)
		{
			bool bActorModified = false;
			bool bActorSelectionChanged = false;
			bChangesOccurred |= UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, /*bActorNotifySelectionChange*/false, /*bActorRedrawViewports*/false);
			bSelectionChanged |= bActorSelectionChanged;
		}
	};

	if (InLevel)
	{
		for (AActor* Actor : InLevel->Actors)
		{
			UpdateActorVisibilityLambda(Actor);
		}
	}
	else if (UWorld* World = GetWorld())
	{
		for (AActor* Actor : FActorRange(World))
		{
			UpdateActorVisibilityLambda(Actor);
		}
	}

	if (bNotifySelectionChange && bSelectionChanged)
	{
		GEditor->NoteSelectionChange();
	}

	if (bRedrawViewports)
	{
		GEditor->RedrawLevelEditingViewports();
	}

	return bChangesOccurred;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on DataLayers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataLayerEditorSubsystem::AppendActorsFromDataLayer(UDataLayerInstance* DataLayerInstance, TArray<AActor*>& InOutActors) const
{
	AppendActorsFromDataLayer(DataLayerInstance, InOutActors, nullptr);
}

void UDataLayerEditorSubsystem::AppendActorsFromDataLayer(UDataLayerInstance* DataLayerInstance, TArray<AActor*>& InOutActors, const TSharedPtr<FActorFilter>& Filter) const
{
	AppendActorsFromDataLayers({ DataLayerInstance }, InOutActors, Filter);
}

void UDataLayerEditorSubsystem::AppendActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, TArray<AActor*>& InOutActors) const
{
	AppendActorsFromDataLayers(DataLayerInstances, InOutActors, nullptr);
}

void UDataLayerEditorSubsystem::AppendActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, TArray<AActor*>& InOutActors, const TSharedPtr<FActorFilter>& Filter) const
{
	for (AActor* Actor : FActorRange(GetWorld()))
	{
		if (Filter.IsValid() && !Filter->PassesFilter(Actor))
		{
			continue;
		}
		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			if (DataLayerInstance)
			{
				if (Actor->ContainsDataLayer(DataLayerInstance) || Actor->GetDataLayerInstancesForLevel().Contains(DataLayerInstance))
				{
					InOutActors.Add(Actor);
					break;
				}
			}
		}
	}
}

TArray<AActor*> UDataLayerEditorSubsystem::GetActorsFromDataLayer(UDataLayerInstance* DataLayerInstance) const
{
	TArray<AActor*> OutActors;
	AppendActorsFromDataLayer(DataLayerInstance, OutActors);
	return OutActors;
}

TArray<AActor*> UDataLayerEditorSubsystem::GetActorsFromDataLayer(UDataLayerInstance* DataLayerInstance, const TSharedPtr<FActorFilter>& Filter) const
{
	TArray<AActor*> OutActors;
	AppendActorsFromDataLayer(DataLayerInstance, OutActors, Filter);
	return OutActors;
}

TArray<AActor*> UDataLayerEditorSubsystem::GetActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers) const
{
	TArray<AActor*> OutActors;
	AppendActorsFromDataLayers(DataLayers, OutActors);
	return OutActors;
}

TArray<AActor*> UDataLayerEditorSubsystem::GetActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers, const TSharedPtr<FActorFilter>& Filter) const
{
	TArray<AActor*> OutActors;
	AppendActorsFromDataLayers(DataLayers, OutActors, Filter);
	return OutActors;
}

void UDataLayerEditorSubsystem::SetDataLayerVisibility(UDataLayerInstance* DataLayerInstance, const bool bIsVisible)
{
	SetDataLayersVisibility({ DataLayerInstance }, bIsVisible);
}

void UDataLayerEditorSubsystem::SetDataLayersVisibility(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsVisible)
{
	bool bChangeOccurred = false;
	for (UDataLayerInstance* DataLayerInstance : DataLayers)
	{
		check(DataLayerInstance);

		if (DataLayerInstance->IsVisible() != bIsVisible)
		{
			DataLayerInstance->Modify(/*bAlswaysMarkDirty*/false);
			DataLayerInstance->SetVisible(bIsVisible);
			BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, "bIsVisible");
			bChangeOccurred = true;
		}
	}

	if (bChangeOccurred)
	{
		UpdateAllActorsVisibility(true, true);
	}
}

void UDataLayerEditorSubsystem::ToggleDataLayerVisibility(UDataLayerInstance* DataLayerInstance)
{
	check(DataLayerInstance);
	SetDataLayerVisibility(DataLayerInstance, !DataLayerInstance->IsVisible());
}

void UDataLayerEditorSubsystem::ToggleDataLayersVisibility(const TArray<UDataLayerInstance*>& DataLayers)
{
	if (DataLayers.Num() == 0)
	{
		return;
	}

	for (UDataLayerInstance* DataLayerInstance : DataLayers)
	{
		DataLayerInstance->Modify();
		DataLayerInstance->SetVisible(!DataLayerInstance->IsVisible());
		BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, "bIsVisible");
	}

	UpdateAllActorsVisibility(true, true);
}

void UDataLayerEditorSubsystem::MakeAllDataLayersVisible()
{
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->ForEachDataLayerInstance([this](UDataLayerInstance* DataLayerInstance)
		{
			if (!DataLayerInstance->IsVisible())
			{
				DataLayerInstance->Modify();
				DataLayerInstance->SetVisible(true);
				BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, "bIsVisible");
			}
			return true;
		});
	
	UpdateAllActorsVisibility(true, true);
}
}

bool UDataLayerEditorSubsystem::SetDataLayerIsLoadedInEditorInternal(UDataLayerInstance* DataLayerInstance, const bool bIsLoadedInEditor, const bool bIsFromUserChange)
{
	check(DataLayerInstance);
	if (DataLayerInstance->IsLoadedInEditor() != bIsLoadedInEditor)
	{
		const bool bWasVisible = DataLayerInstance->IsEffectiveVisible();

		DataLayerInstance->Modify(false);
		DataLayerInstance->SetIsLoadedInEditor(bIsLoadedInEditor, /*bFromUserChange*/bIsFromUserChange);
		BroadcastDataLayerChanged(EDataLayerAction::Modify, DataLayerInstance, "bIsLoadedInEditor");

		if (DataLayerInstance->IsEffectiveVisible() != bWasVisible)
		{
			UpdateAllActorsVisibility(true, true);
		}
		return true;
	}
	return false;
}

bool UDataLayerEditorSubsystem::SetDataLayerIsLoadedInEditor(UDataLayerInstance* DataLayerInstance, const bool bIsLoadedInEditor, const bool bIsFromUserChange)
{
	if (SetDataLayerIsLoadedInEditorInternal(DataLayerInstance, bIsLoadedInEditor, bIsFromUserChange))
	{
		OnDataLayerEditorLoadingStateChanged(bIsFromUserChange);
	}
	return true;
}

bool UDataLayerEditorSubsystem::SetDataLayersIsLoadedInEditor(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsLoadedInEditor, const bool bIsFromUserChange)
{
	bool bChanged = false;
	for (UDataLayerInstance* DataLayerInstance : DataLayers)
	{
		bChanged |= SetDataLayerIsLoadedInEditorInternal(DataLayerInstance, bIsLoadedInEditor, bIsFromUserChange);
	}
	
	if (bChanged)
	{
		OnDataLayerEditorLoadingStateChanged(bIsFromUserChange);
	}

	return true;
}

bool UDataLayerEditorSubsystem::ToggleDataLayerIsLoadedInEditor(UDataLayerInstance* DataLayerInstance, const bool bIsFromUserChange)
{
	check(DataLayerInstance);
	return SetDataLayerIsLoadedInEditor(DataLayerInstance, !DataLayerInstance->IsLoadedInEditor(), bIsFromUserChange);
}

bool UDataLayerEditorSubsystem::ToggleDataLayersIsLoadedInEditor(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsFromUserChange)
{
	bool bChanged = false;
	for (UDataLayerInstance* DataLayerInstance : DataLayers)
	{
		bChanged |= SetDataLayerIsLoadedInEditorInternal(DataLayerInstance, !DataLayerInstance->IsLoadedInEditor(), bIsFromUserChange);
	}
	
	if (bChanged)
	{
		OnDataLayerEditorLoadingStateChanged(bIsFromUserChange);
	}

	return true;
}

TArray<UDataLayerInstance*> UDataLayerEditorSubsystem::GetAllDataLayers()
{
	TArray<UDataLayerInstance*> DataLayerInstances;
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->ForEachDataLayerInstance([&DataLayerInstances](UDataLayerInstance* DataLayerInstance)
		{
			DataLayerInstances.Add(DataLayerInstance);
			return true;
		});
	}
	return DataLayerInstances;
}

bool UDataLayerEditorSubsystem::ResetUserSettings()
{
	bool bChanged = false;
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->ForEachDataLayerInstance([this, &bChanged](UDataLayerInstance* DataLayerInstance)
		{
			bChanged |= SetDataLayerIsLoadedInEditorInternal(DataLayerInstance, DataLayerInstance->IsInitiallyLoadedInEditor(), true);
			return true;
		});
	
		if (bChanged)
		{
			OnDataLayerEditorLoadingStateChanged(true);
		}
	}
	return true;
}

bool UDataLayerEditorSubsystem::PassDataLayersFilter(UWorld* World, const FWorldPartitionHandle& ActorHandle)
{
	UWorld* OwningWorld = World->PersistentLevel->GetWorld();

	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(OwningWorld))
	{
		// If Actor is loaded and dirty, use a newly resolved DataLayerInstanceNames array
		auto GetLatestDataLayerInstanceNames = [DataLayerManager, ActorHandle]() -> FDataLayerInstanceNames
		{
			if(AActor* Actor = ActorHandle->GetActor(false); Actor && Actor->GetPackage()->IsDirty())
			{
				TUniquePtr<FWorldPartitionActorDesc> NewActorDesc = Actor->CreateActorDesc();
				return FDataLayerUtils::ResolveDataLayerInstanceNames(DataLayerManager, NewActorDesc.Get());
			}
			else
			{
				return ActorHandle->GetDataLayerInstanceNames();
			}
		};

		if (IsRunningCookCommandlet())
		{
			// When running cook commandlet, dont allow loading of actors with runtime loaded data layers
			for (const FName& DataLayerInstanceName : GetLatestDataLayerInstanceNames().ToArray())
			{
				const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerInstanceName);
				if (DataLayerInstance && DataLayerInstance->IsRuntime())
				{
					return false;
				}
			}

			return true;
		}

		return DataLayerManager->ResolveIsLoadedInEditor(GetLatestDataLayerInstanceNames().ToArray());
	}

	return true;
}

UDataLayerInstance* UDataLayerEditorSubsystem::GetDataLayerInstance(const FName& DataLayerInstanceName) const
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		return const_cast<UDataLayerInstance*>(DataLayerManager->GetDataLayerInstance(DataLayerInstanceName));
	}
	return nullptr;
}

UDataLayerInstance* UDataLayerEditorSubsystem::GetDataLayerInstance(const UDataLayerAsset* DataLayerAsset) const
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		return const_cast<UDataLayerInstance*>(DataLayerManager->GetDataLayerInstance(DataLayerAsset));
	}
	return nullptr;
}

TArray<UDataLayerInstance*> UDataLayerEditorSubsystem::GetDataLayerInstances(const TArray<UDataLayerAsset*> DataLayerAssets) const
{
	TArray<const UDataLayerAsset*> ConstAssets;
	Algo::Transform(DataLayerAssets, ConstAssets, [](UDataLayerAsset* DataLayerAsset) { return DataLayerAsset; });

	TArray<const UDataLayerInstance*> DataLayerInstances = GetDataLayerInstances(ConstAssets);

	TArray<UDataLayerInstance*> Result;
	Algo::Transform(DataLayerInstances, Result, [](const UDataLayerInstance* DataLayerInstance) { return const_cast<UDataLayerInstance*>(DataLayerInstance); });
	return Result;
}

void UDataLayerEditorSubsystem::AddAllDataLayersTo(TArray<TWeakObjectPtr<UDataLayerInstance>>& OutDataLayers) const
{
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->ForEachDataLayerInstance([&OutDataLayers](UDataLayerInstance* DataLayerInstance)
		{
			OutDataLayers.Add(DataLayerInstance);
			return true;
		});
	}
}

UDataLayerInstance* UDataLayerEditorSubsystem::CreateDataLayerInstance(const FDataLayerCreationParameters& Parameters)
{
	UDataLayerInstance* NewDataLayerInstance = nullptr;

	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));

	if (AWorldDataLayers* WorldDataLayers = Parameters.WorldDataLayers != nullptr ? Parameters.WorldDataLayers.Get() : (GetWorld() ? GetWorld()->GetWorldDataLayers() : nullptr))
	{
		if (Parameters.bIsPrivate || !Parameters.DataLayerAsset)
		{
			ensureMsgf(Parameters.bIsPrivate, TEXT("When a DataLayerAsset is not provided, bIsPrivate should be set to true for clarity"));
			NewDataLayerInstance = CreateDataLayerInstance<UDataLayerInstancePrivate>(WorldDataLayers);
		}
		else
		{
			if (UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(Parameters.DataLayerAsset))
			{
				UWorld* OuterWorld = WorldDataLayers->GetTypedOuter<UWorld>();
				if (UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(OuterWorld))
				{
					FText FailureReason;
					if (ExternalDataLayerManager->CanInjectExternalDataLayerAsset(ExternalDataLayerAsset, &FailureReason))
					{
						const bool bAllowCreate = true;
						if (AWorldDataLayers* EDLWorldDataLayers = ExternalDataLayerManager->GetWorldDataLayers(ExternalDataLayerAsset, bAllowCreate))
						{
							UExternalDataLayerInstance* ExternalDataLayerInstance = EDLWorldDataLayers->GetExternalDataLayerInstance(ExternalDataLayerAsset);
							if (ensure(!ExternalDataLayerInstance))
							{
								// Create External Data Layer Instance
								ExternalDataLayerInstance = EDLWorldDataLayers->CreateDataLayer<UExternalDataLayerInstance>(ExternalDataLayerAsset);
								if (ensure(ExternalDataLayerInstance))
								{
									if (ExternalDataLayerManager->InjectExternalDataLayer(ExternalDataLayerAsset))
									{
										NewDataLayerInstance = const_cast<UExternalDataLayerInstance*>(ExternalDataLayerManager->GetExternalDataLayerInstance(ExternalDataLayerAsset));
										ensure(NewDataLayerInstance == ExternalDataLayerInstance);
									}
									else
									{
										UE_LOGF(LogDataLayerEditorSubsystem, Warning, "[EDL %ls] Failed to inject External Data Layer.", *ExternalDataLayerAsset->GetName());
									}
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogDataLayerEditorSubsystem, Warning, "%ls", *FailureReason.ToString());
						LastWarningNotification = FailureReason;
					}
				}
				UE_CLOGF(!NewDataLayerInstance, LogDataLayerEditorSubsystem, Error, "[EDL %ls] Failed to create External Data Layer Instance.", *ExternalDataLayerAsset->GetName());
			}
			else
			{
				NewDataLayerInstance = CreateDataLayerInstance<UDataLayerInstanceWithAsset>(WorldDataLayers, Parameters.DataLayerAsset);
			}
		}
	}

	if (const UExternalDataLayerInstance* RootExternalDataLayerInstance = NewDataLayerInstance ? NewDataLayerInstance->GetDirectOuterWorldDataLayers()->GetRootExternalDataLayerInstance() : nullptr)
	{
		UExternalDataLayerInstance* Parent = const_cast<UExternalDataLayerInstance*>(RootExternalDataLayerInstance);
		if ((NewDataLayerInstance != Parent) && !NewDataLayerInstance->SetParent(Parent))
		{
			FText Reason;
			if (!NewDataLayerInstance->CanBeChildOf(Parent, &Reason))
			{
				UE_LOGF(LogDataLayerEditorSubsystem, Warning, "Can't create Data Layer Instance %ls under %ls : %ls", *NewDataLayerInstance->GetDataLayerShortName(), *Parent->GetDataLayerShortName(), *Reason.ToString());
			}
			// Failed to root Data Layer Instance under root EDL
			DeleteDataLayer(NewDataLayerInstance);
			NewDataLayerInstance = nullptr;
		}
	}

	if (NewDataLayerInstance)
	{
		BroadcastDataLayerChanged(EDataLayerAction::Add, NewDataLayerInstance, NAME_None);
	}
	
	return NewDataLayerInstance;
}

void UDataLayerEditorSubsystem::DeleteDataLayers(const TArray<UDataLayerInstance*>& DataLayersToDelete)
{
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	
	TArray<UDataLayerInstance*> DeletedDataLayerInstances;
	for (UDataLayerInstance* DataLayerToDelete : DataLayersToDelete)
	{
		if (!DataLayerToDelete)
		{
			continue;
		}

		if (!DataLayerToDelete->CanBeRemoved())
		{
			continue;
		}

		AWorldDataLayers* OuterWorldDataLayers = DataLayerToDelete->GetDirectOuterWorldDataLayers();
		if (OuterWorldDataLayers->RemoveDataLayer(DataLayerToDelete))
		{
			DeletedDataLayerInstances.Add(DataLayerToDelete);
		}
	}
	for (UDataLayerInstance* DeletedDataLayerInstance : DeletedDataLayerInstances)
	{
		BroadcastDataLayerChanged(EDataLayerAction::Delete, DeletedDataLayerInstance, NAME_None);
	}
}

void UDataLayerEditorSubsystem::DeleteDataLayer(UDataLayerInstance* DataLayerToDelete)
{
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));

	if (!DataLayerToDelete)
	{
		return;
	}
	
	if (!DataLayerToDelete->CanBeRemoved())
	{
		return;
	}

	AWorldDataLayers* OuterWorldDataLayers = DataLayerToDelete->GetDirectOuterWorldDataLayers();
	if (OuterWorldDataLayers->RemoveDataLayer(DataLayerToDelete))
	{
		BroadcastDataLayerChanged(EDataLayerAction::Delete, DataLayerToDelete, NAME_None);
	}
}

void UDataLayerEditorSubsystem::BroadcastActorDataLayersChanged(const TWeakObjectPtr<AActor>& ChangedActor)
{
	bRebuildSelectedDataLayersFromEditorSelection = true;
	ActorDataLayersChanged.Broadcast(ChangedActor);
}

void UDataLayerEditorSubsystem::BroadcastDataLayerChanged(const EDataLayerAction Action, const TWeakObjectPtr<const UDataLayerInstance>& ChangedDataLayer, const FName& ChangedProperty)
{
	bRebuildSelectedDataLayersFromEditorSelection = true;
	DataLayerChanged.Broadcast(Action, ChangedDataLayer, ChangedProperty);

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (UWorld* World = GetWorld())
	{
		FActorPrimitiveColorHandler::Get().RefreshPrimitiveColorHandler(NAME_RuntimeDataLayerColor, World);
		FActorPrimitiveColorHandler::Get().RefreshPrimitiveColorHandler(NAME_CurrentDataLayerColor, World);
		FActorPrimitiveColorHandler::Get().RefreshPrimitiveColorHandler(NAME_ExternalDataLayerColor, World);
	}
#endif
}

void UDataLayerEditorSubsystem::OnDataLayerEditorLoadingStateChanged(bool bIsFromUserChange)
{
	FScopedSlowTask SlowTask(1, LOCTEXT("UpdatingLoadedActors", "Updating loaded actors..."));
	SlowTask.MakeDialog();

	BroadcastDataLayerEditorLoadingStateChanged(bIsFromUserChange);
}

void UDataLayerEditorSubsystem::BroadcastDataLayerEditorLoadingStateChanged(bool bIsFromUserChange)
{
	UE_CLOGF(!GetWorld(), LogDataLayerEditorSubsystem, Error, "%ls - Failed because world in null.", ANSI_TO_TCHAR(__FUNCTION__));
	DataLayerEditorLoadingStateChanged.Broadcast(bIsFromUserChange);
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->UpdateDataLayerEditorPerProjectUserSettings();
	}
}

void UDataLayerEditorSubsystem::OnSelectionChanged()
{
	bRebuildSelectedDataLayersFromEditorSelection = true;
}

const UExternalDataLayerAsset* UDataLayerEditorSubsystem::GetReferencingWorldSurrogateObjectForObject(UWorld* ReferencingWorld, const FSoftObjectPath& ObjectPath)
{
	const UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(ReferencingWorld);
	if (const UExternalDataLayerAsset* ExternalDataLayerAsset = ExternalDataLayerManager ? ExternalDataLayerManager->GetMatchingExternalDataLayerAssetForObjectPath(ObjectPath) : nullptr)
	{
		const UDataLayerInstance* ExternalDataLayerInstance = UDataLayerManager::GetDataLayerManager(ReferencingWorld)->GetDataLayerInstance(ExternalDataLayerAsset);
		if (ExternalDataLayerInstance && ExternalDataLayerInstance->CanBeInActorEditorContext())
		{
			return ExternalDataLayerAsset;
		}
	}
	return nullptr;
}

struct FExternalDataLayerWorldSurrogateReferencingObject : public FLevelEditorDragDropWorldSurrogateReferencingObject
{
public:
	FExternalDataLayerWorldSurrogateReferencingObject(const UObject* InSurrogateObject) 
		: FLevelEditorDragDropWorldSurrogateReferencingObject(InSurrogateObject)
	{}

	virtual bool OnPreDropObjects(UWorld* World, const TArray<UObject*>& DroppedObjects)
	{
		if (!FLevelEditorDragDropWorldSurrogateReferencingObject::OnPreDropObjects(World, DroppedObjects))
		{
			return false;
		}
		
		ULevel* CurrentLevel = World->GetCurrentLevel();
		UWorld* CurrentLevelOuterWorld = CurrentLevel ? Cast<UWorld>(CurrentLevel->GetOuter()) : nullptr;
		UWorld* ReferencingWorld = CurrentLevelOuterWorld ? CurrentLevelOuterWorld : World;
		
		const UExternalDataLayerAsset* ExternalDataLayerAsset = UDataLayerEditorSubsystem::GetReferencingWorldSurrogateObjectForObject(ReferencingWorld, FSoftObjectPath(DroppedObjects[0]));
		EDLContext = ExternalDataLayerAsset ? TUniquePtr<FScopedOverrideSpawningLevelMountPointObject>(new FScopedOverrideSpawningLevelMountPointObject(ExternalDataLayerAsset)) : nullptr;
		return EDLContext.IsValid() && (ULevel::GetOverrideSpawningLevelMountPointObject() == ExternalDataLayerAsset);
	}

	virtual bool OnPostDropObjects(UWorld* World, const TArray<UObject*>& DroppedObjects)
	{
		if (!FLevelEditorDragDropWorldSurrogateReferencingObject::OnPostDropObjects(World, DroppedObjects))
		{
			return false;
		}
		EDLContext.Reset();
		return true;
	}

private:
	TUniquePtr<FScopedOverrideSpawningLevelMountPointObject> EDLContext;
};

TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> UDataLayerEditorSubsystem::OnLevelEditorDragDropWorldSurrogateReferencingObject(UWorld* ReferencingWorld, const FSoftObjectPath& Object)
{
	// For backward compatibility, don't try to find a world surrogate object when there's a Content Bundle in the actor editor context
	if (!IWorldPartitionEditorModule::Get().IsEditingContentBundle())
	{
		if (const UExternalDataLayerAsset* ExternalDataLayerAsset = GetReferencingWorldSurrogateObjectForObject(ReferencingWorld, Object))
		{
			return MakeUnique<FExternalDataLayerWorldSurrogateReferencingObject>(ExternalDataLayerAsset);
		}
	}
	return nullptr;
}

void UDataLayerEditorSubsystem::OnExternalDataLayerAssetRegistrationStateChanged(const UExternalDataLayerAsset* ExternalDataLayerAsset, EExternalDataLayerRegistrationState OldState, EExternalDataLayerRegistrationState NewState)
{
	if (ExternalDataLayerAsset)
	{
		EditorRefreshDataLayerBrowser();
	}
}

const TSet<TWeakObjectPtr<const UDataLayerInstance>>& UDataLayerEditorSubsystem::GetSelectedDataLayersFromEditorSelection() const
{
	if (bRebuildSelectedDataLayersFromEditorSelection)
	{
		bRebuildSelectedDataLayersFromEditorSelection = false;

		SelectedDataLayersFromEditorSelection.Reset();
		TArray<AActor*> Actors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(Actors);
		for (const AActor* Actor : Actors)
		{
			for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstances())
			{
				SelectedDataLayersFromEditorSelection.Add(DataLayerInstance);
			}
			for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstancesForLevel())
			{
				SelectedDataLayersFromEditorSelection.Add(DataLayerInstance);
			}
		}
	}
	return SelectedDataLayersFromEditorSelection;
}

bool UDataLayerEditorSubsystem::SetDataLayerShortName(UDataLayerInstance* DataLayerInstance, const FString& InNewShortName)
{
	if (DataLayerInstance->CanEditDataLayerShortName())
	{
		if (FDataLayerUtils::SetDataLayerShortName(DataLayerInstance, InNewShortName))
		{
			BroadcastDataLayerChanged(EDataLayerAction::Rename, DataLayerInstance, "DataLayerShortName");
			return true;
		}
	}

	return false;
}

bool UDataLayerEditorSubsystem::MoveActorToDataLayers(AActor* InActor, const TArray<UDataLayerInstance*>& InDataLayerInstances)
{
	UWorld* World = GetWorld();
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(World);
	if (!DataLayerManager)
	{
		return false;
	}

	TSet<UDataLayerInstance*> AllDataLayerInstance(DataLayerManager->GetDataLayerInstances());
	for (UDataLayerInstance* DataLayerInstance : InDataLayerInstances)
	{
		if (!AllDataLayerInstance.Contains(DataLayerInstance))
		{
			UE_LOGF(LogWorldPartition, Warning, "Can't move actor %ls to Data Layers: Invalid Data Layer %ls", *InActor->GetActorNameOrLabel(), *DataLayerInstance->GetDataLayerShortName());
			return false;
		}
	}

	TSet<const UDataLayerInstance*> ActorDataLayers(InActor->GetDataLayerInstances());
	TSet<const UDataLayerInstance*> NewDataLayers(InDataLayerInstances);
	if ((ActorDataLayers.Num() == NewDataLayers.Num()) &&
		(ActorDataLayers.Intersect(NewDataLayers).Num() == ActorDataLayers.Num()))
	{
		return true;
	}

	// Data Layers of partition actors impact their name. In this case, use a dedicated function that will do the move operation.
	APartitionActor* PartitionActor = Cast<APartitionActor>(InActor);
	if (PartitionActor && PartitionActor->IsPartitionActorNameAffectedByDataLayers())
	{
		UActorPartitionSubsystem* ActorPartitionSubsystem = UWorld::GetSubsystem<UActorPartitionSubsystem>(World);
		if (!ActorPartitionSubsystem || !ActorPartitionSubsystem->MoveActorToDataLayers(PartitionActor, InDataLayerInstances))
		{
			return false;
		}
	}
	else
	{
		// Test moving from/to new External Data Layer
		const UExternalDataLayerAsset* OldExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
		const UDataLayerInstance* const* ExternalDataLayerInstance = Algo::FindByPredicate(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); });
		const UExternalDataLayerInstance* NewExternalDataLayerInstance = ExternalDataLayerInstance ? Cast<UExternalDataLayerInstance>(*ExternalDataLayerInstance) : nullptr;
		const UExternalDataLayerAsset* NewExternalDataLayerAsset = NewExternalDataLayerInstance ? NewExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
		constexpr bool bAllowNonUserManaged = true;
		FMoveToExternalDataLayerParams Params(NewExternalDataLayerInstance, bAllowNonUserManaged);
		if ((OldExternalDataLayerAsset != NewExternalDataLayerAsset))
		{
			FText FailureReason;
			if (!FExternalDataLayerHelper::CanMoveActorsToExternalDataLayer({ InActor }, Params, &FailureReason))
			{
				UE_LOGF(LogWorldPartition, Warning, "Can't move actor %ls to External Data Layer. %ls", *InActor->GetName(), *FailureReason.ToString());
				return false;
			}
		}

		// Move to new Data Layers:
		// First, remove all non-EDL layers
		const bool bIncludeExternalDataLayerAsset = false;
		for (const UDataLayerAsset* DataLayerAsset : InActor->GetDataLayerAssets(bIncludeExternalDataLayerAsset))
		{
			FAssignActorDataLayer::RemoveDataLayerAsset(InActor, DataLayerAsset);
		}
		// Second, move to new External Data Layer if any or different. Subsequent valid data layers should be in that EDL, and the EDL must be added first for package paths to allow adding the DLs.
		if (OldExternalDataLayerAsset != NewExternalDataLayerAsset)
		{
			verify(FExternalDataLayerHelper::MoveActorsToExternalDataLayer({ InActor }, Params));
		}
		// Finally, apply the rest of the layers
		for (UDataLayerInstance* DataLayerInstance : InDataLayerInstances)
		{
			if (!DataLayerInstance->IsA<UExternalDataLayerInstance>())
			{
				AddActorToDataLayer(InActor, DataLayerInstance);
			}
		}
	}
	return true;
}

bool UDataLayerEditorSubsystem::ApplyActorEditorContextDataLayersToActors(const TArray<AActor*>& InActors)
{
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	if (!DataLayerManager)
	{
		return false;
	}
	TArray<UDataLayerInstance*> DataLayerInstances = DataLayerManager->GetActorEditorContextDataLayers();

	bool bSuccess = true;
	for (AActor* Actor : InActors)
	{
		if (!MoveActorToDataLayers(Actor, DataLayerInstances))
		{
			bSuccess = false;
		}
	}
	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
