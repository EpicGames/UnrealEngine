// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/DataLayerFactory.h"

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerFactory)

UDataLayerFactory::UDataLayerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDataLayerAsset::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UDataLayerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UDataLayerAsset* DataLayerAsset = NewObject<UDataLayerAsset>(InParent, InName, Flags);
	DataLayerAsset->OnCreated();
	return DataLayerAsset;
}
