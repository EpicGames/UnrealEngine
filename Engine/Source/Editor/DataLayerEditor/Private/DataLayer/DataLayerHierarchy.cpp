// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerHierarchy.h"

#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayerActorTreeItem.h"
#include "DataLayerMode.h"
#include "DataLayerTreeItem.h"
#include "DataLayersActorDescTreeItem.h"
#include "Delegates/Delegate.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "ISceneOutlinerMode.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "WorldDataLayersTreeItem.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

TUniquePtr<FDataLayerHierarchy> FDataLayerHierarchy::Create(FDataLayerMode* Mode, const TWeakObjectPtr<UWorld>& World)
{
	return TUniquePtr<FDataLayerHierarchy>(new FDataLayerHierarchy(Mode, World));
}

FDataLayerHierarchy::FDataLayerHierarchy(FDataLayerMode* Mode, const TWeakObjectPtr<UWorld>& World)
	: ISceneOutlinerHierarchy(Mode)
	, RepresentingWorld(World)
	, bShowEditorDataLayers(true)
	, bShowRuntimeDataLayers(true)
	, bShowDataLayerActors(true)
	, bShowUnloadedActors(true)
	, bShowOnlySelectedActors(false)
	, bHighlightSelectedDataLayers(false)
	, bShowLevelInstanceContent(false)
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().AddRaw(this, &FDataLayerHierarchy::OnLevelActorAdded);
		GEngine->OnLevelActorDeleted().AddRaw(this, &FDataLayerHierarchy::OnLevelActorDeleted);
		GEngine->OnLevelActorListChanged().AddRaw(this, &FDataLayerHierarchy::OnLevelActorListChanged);
	}

	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().AddRaw(this, &FDataLayerHierarchy::OnWorldPartitionCreated);

	if (World.IsValid())
	{
		if (World->PersistentLevel)
		{
			World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddRaw(this, &FDataLayerHierarchy::OnLoadedActorAdded);
			World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddRaw(this, &FDataLayerHierarchy::OnLoadedActorRemoved);
		}

		World->OnWorldPartitionInitialized().AddRaw(this, &FDataLayerHierarchy::OnWorldPartitionInitialized);
		World->OnWorldPartitionUninitialized().AddRaw(this, &FDataLayerHierarchy::OnWorldPartitionUninitialized);
		
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceAddedEvent.AddRaw(this, &FDataLayerHierarchy::OnActorDescInstanceAdded);
			WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(this, &FDataLayerHierarchy::OnActorDescInstanceRemoved);
		}
	}

	UDataLayerEditorSubsystem::Get()->OnDataLayerChanged().AddRaw(this, &FDataLayerHierarchy::OnDataLayerChanged);
	UDataLayerEditorSubsystem::Get()->OnActorDataLayersChanged().AddRaw(this, &FDataLayerHierarchy::OnActorDataLayersChanged);
	FWorldDelegates::LevelAddedToWorld.AddRaw(this, &FDataLayerHierarchy::OnLevelAdded);
	FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FDataLayerHierarchy::OnLevelRemoved);
}

FDataLayerHierarchy::~FDataLayerHierarchy()
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
	}

	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().RemoveAll(this);

	if (RepresentingWorld.IsValid())
	{
		if (RepresentingWorld->PersistentLevel)
		{
			RepresentingWorld->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
			RepresentingWorld->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.RemoveAll(this);
		}

		RepresentingWorld->OnWorldPartitionInitialized().RemoveAll(this);
		RepresentingWorld->OnWorldPartitionUninitialized().RemoveAll(this);
		
		if (UWorldPartition* WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceAddedEvent.RemoveAll(this);
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}

	if (UDataLayerEditorSubsystem* DataLayerEditorSubSystem = UDataLayerEditorSubsystem::Get())
	{
		DataLayerEditorSubSystem->OnDataLayerChanged().RemoveAll(this);
		DataLayerEditorSubSystem->OnActorDataLayersChanged().RemoveAll(this);
	}
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
}


bool FDataLayerHierarchy::IsDataLayerPartOfSelection(const UDataLayerInstance* DataLayerInstance) const
{
	if (!bShowOnlySelectedActors)
	{
		return true;
	}

	if (UDataLayerEditorSubsystem::Get()->DoesDataLayerContainSelectedActors(DataLayerInstance))
	{
		return true;
	}

	bool bFoundSelected = false;
	DataLayerInstance->ForEachChild([this, &bFoundSelected](const UDataLayerInstance* Child)
	{
		bFoundSelected = IsDataLayerPartOfSelection(Child);
		return !bFoundSelected; // Continue iterating if not found
	});
	return bFoundSelected;
};

FSceneOutlinerTreeItemPtr FDataLayerHierarchy::CreateDataLayerTreeItem(UDataLayerInstance* InDataLayer, bool bInForce) const
{
	FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FDataLayerTreeItem>(InDataLayer, bInForce);
	if (FDataLayerTreeItem* DataLayerTreeItem = Item ? Item->CastTo<FDataLayerTreeItem>() : nullptr)
	{
		DataLayerTreeItem->SetIsHighlightedIfSelected(bHighlightSelectedDataLayers);
	}
	return Item;
}

UWorld* FDataLayerHierarchy::GetOwningWorld() const
{
	UWorld* World = RepresentingWorld.Get();
	return World ? World->PersistentLevel->GetWorld() : nullptr;
}

void FDataLayerHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	auto IsDataLayerShown = [this](const UDataLayerInstance* DataLayerInstance)
	{
		const bool bIsRuntimeDataLayer = DataLayerInstance->IsRuntime();
		return ((bIsRuntimeDataLayer && bShowRuntimeDataLayers) || (!bIsRuntimeDataLayer && bShowEditorDataLayers)) && IsDataLayerPartOfSelection(DataLayerInstance);
	};

	auto CreateDataLayerTreeItems = [this, &OutItems, IsDataLayerShown](const UDataLayerManager* InDataLayerManager)
	{
		if (InDataLayerManager)
		{
			if (AWorldDataLayers* WorldDataLayers = InDataLayerManager->GetWorldDataLayers())
			{
				if (FSceneOutlinerTreeItemPtr WorldDataLayersItem = Mode->CreateItemFor<FWorldDataLayersTreeItem>(WorldDataLayers))
				{
					OutItems.Add(WorldDataLayersItem);
				}
			}

			InDataLayerManager->ForEachDataLayerInstance([this, &OutItems, IsDataLayerShown](UDataLayerInstance* DataLayerInstance)
			{
				if (IsDataLayerShown(DataLayerInstance))
				{
					if (FSceneOutlinerTreeItemPtr DataLayerItem = CreateDataLayerTreeItem(DataLayerInstance))
					{
						OutItems.Add(DataLayerItem);
					}
				}
				return true;
			});
		}
	};

	UWorld* OwningWorld = GetOwningWorld();
	if (!OwningWorld)
	{
		return;
	}

	if (OwningWorld->IsPlayInEditor())
	{
		// PIE loops on each DataLayerManager of each registered WorldPartition. 
		// For performance reasons, child actors are not shown.
		UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(OwningWorld);
		WorldPartitionSubsystem->ForEachWorldPartition([&CreateDataLayerTreeItems](UWorldPartition* WorldPartition)
		{
			CreateDataLayerTreeItems(WorldPartition->GetDataLayerManager());
			return true;
		});
	}
	else
	{
		// CurrentLevel represents the current level if different than the PersistentLevel
		ULevel* CurrentLevel = (OwningWorld->GetCurrentLevel() && !OwningWorld->GetCurrentLevel()->IsPersistentLevel()) ? OwningWorld->GetCurrentLevel() : nullptr;

		// Create DataLayerTreeItems for World and for Current Level (if any).
		CreateDataLayerTreeItems(OwningWorld->GetDataLayerManager());

		if (CurrentLevel)
		{
			CreateDataLayerTreeItems(UDataLayerManager::GetDataLayerManager(CurrentLevel));
		}

		if (bShowDataLayerActors)
		{
			for (AActor* Actor : FActorRange(OwningWorld))
			{
				// Consider all actors or actors part of current level (if there is one)
				bool bConsiderActor = !CurrentLevel || (Actor->GetLevel() == CurrentLevel);
				if (bConsiderActor)
				{
					for (const UDataLayerInstance* DataLayerInstance : (Actor->GetLevel() == CurrentLevel) ? Actor->GetDataLayerInstancesForLevel() : Actor->GetDataLayerInstances())
					{
						if (IsDataLayerShown(DataLayerInstance))
						{
							if (FSceneOutlinerTreeItemPtr DataLayerActorItem = Mode->CreateItemFor<FDataLayerActorTreeItem>(FDataLayerActorTreeItemData(Actor, const_cast<UDataLayerInstance*>(DataLayerInstance))))
							{
								OutItems.Add(DataLayerActorItem);
							}
						}
					}
				}
			}

			if (bShowUnloadedActors)
			{
				// Build mapping of OwninWorld DataLayerInstance to CurrentLevel DataLayerInstance
				TMap<UDataLayerInstance*, UDataLayerInstance*> WorldToLevelDataLayerMap;
				if (CurrentLevel)
				{
					AWorldDataLayers* OwningWorldWorldDataLayers = OwningWorld->GetWorldDataLayers();
					AWorldDataLayers* CurrentLevelWorldDataLayers = CurrentLevel->GetWorldDataLayers();
					if (OwningWorldWorldDataLayers && CurrentLevelWorldDataLayers)
					{
						check(OwningWorldWorldDataLayers != CurrentLevelWorldDataLayers);
						OwningWorldWorldDataLayers->ForEachDataLayerInstance([&WorldToLevelDataLayerMap, CurrentLevelWorldDataLayers](UDataLayerInstance* DataLayerInstance)
						{
							if (UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
							{
								const UDataLayerAsset* DataLayerAsset = DataLayerInstanceWithAsset->GetAsset();
								if (const UDataLayerInstance* LevelDataLayerInstance = DataLayerAsset ? CurrentLevelWorldDataLayers->GetDataLayerInstance(DataLayerAsset) : nullptr)
								{
									WorldToLevelDataLayerMap.Add(DataLayerInstance, const_cast<UDataLayerInstance*>(LevelDataLayerInstance));
								}
							}
							return true;
						});
					}
				}

				UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(OwningWorld);
				const UDataLayerManager* DataLayerManager = OwningWorld->GetDataLayerManager();
				if (WorldPartitionSubsystem && DataLayerManager)
				{
					const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld);
					WorldPartitionSubsystem->ForEachWorldPartition([this, DataLayerManager, CurrentLevel, LevelInstanceSubsystem, IsDataLayerShown, &WorldToLevelDataLayerMap, &OutItems](UWorldPartition* WorldPartition)
					{
						// Skip WorldPartition if it's not the one of the current level (the editing level instance)
						// or if we hide the content of level instances
						UWorld* OuterWorld = WorldPartition->GetTypedOuter<UWorld>();
						ULevel* OuterLevel = OuterWorld ? OuterWorld->PersistentLevel.Get() : nullptr;
						if ((CurrentLevel && (CurrentLevel != OuterLevel)) ||
							(!bShowLevelInstanceContent && OuterLevel && LevelInstanceSubsystem && LevelInstanceSubsystem->GetOwningLevelInstance(OuterLevel)))
						{
							return true;
						}

						const TMap<FGuid, AActor*> LoadedActors = FWorldPartitionHelpers::GetLoadedActorsForLevel(OuterLevel);
						const TMap<FGuid, AActor*> RegisteredActors = FWorldPartitionHelpers::GetRegisteredActorsForLevel(OuterLevel);

						// Create an FDataLayerActorDescTreeItem for each unloaded actor of this WorldPartition
						FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [this, IsDataLayerShown, DataLayerManager, CurrentLevel, &WorldToLevelDataLayerMap, &LoadedActors, &RegisteredActors, &OutItems](const FWorldPartitionActorDescInstance* ActorDescInstance)
						{
							if (ActorDescInstance && FActorDescTreeItem::ShouldDisplayInOutliner(ActorDescInstance))
							{
								if (AActor* const* LoadedActor = LoadedActors.Find(ActorDescInstance->GetGuid()))
								{
									if (!IsValid(*LoadedActor) || RegisteredActors.Contains(ActorDescInstance->GetGuid()))
									{
										return true;
									}
								}

								for (const FName& DataLayerInstanceName : ActorDescInstance->GetDataLayerInstanceNames().ToArray())
								{
									if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerInstanceName))
									{
										if (IsDataLayerShown(DataLayerInstance))
										{
											UDataLayerInstance* EffectiveDataLayerInstance = const_cast<UDataLayerInstance*>(DataLayerInstance);
											// If current level is valid, build Item with current level's DataLayerInstance
											if (CurrentLevel)
											{
												UDataLayerInstance** LevelDataLayerInstance = WorldToLevelDataLayerMap.Find(DataLayerInstance);
												EffectiveDataLayerInstance = *LevelDataLayerInstance;
											}
											if (const FSceneOutlinerTreeItemPtr ActorDescItem = Mode->CreateItemFor<FDataLayerActorDescTreeItem>(FDataLayerActorDescTreeItemData(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance(), EffectiveDataLayerInstance)))
											{
												OutItems.Add(ActorDescItem);
											}
										}
									}
								}
							}
							return true;
						});
						return true;
					});
				}
			}
		}
	}
}

FSceneOutlinerTreeItemPtr FDataLayerHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	if (const FDataLayerTreeItem* DataLayerTreeItem = Item.CastTo<FDataLayerTreeItem>())
	{
		if (UDataLayerInstance* DataLayerInstance = DataLayerTreeItem->GetDataLayer())
		{
			if (UDataLayerInstance* ParentDataLayer = DataLayerInstance->GetParent())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentDataLayer))
				{
					return *ParentItem;
				}
				else if (bCreate)
				{
					return CreateDataLayerTreeItem(ParentDataLayer, true);
				}
			}
			else
			{
				// Parent WorldDataLayers
				if (AWorldDataLayers* OuterWorldDataLayers = DataLayerInstance->GetOuterWorldDataLayers())
				{
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OuterWorldDataLayers))
					{
						return *ParentItem;
					}
					else if (bCreate)
					{
						return Mode->CreateItemFor<FWorldDataLayersTreeItem>(OuterWorldDataLayers, true);
					}
				}
			}
		}
	}
	else if (const FDataLayerActorTreeItem* DataLayerActorTreeItem = Item.CastTo<FDataLayerActorTreeItem>())
	{
		if (UDataLayerInstance* DataLayerInstance = DataLayerActorTreeItem->GetDataLayer())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(DataLayerInstance))
			{
				return *ParentItem;
			}
			else if (bCreate)
			{
				return CreateDataLayerTreeItem(DataLayerInstance, true);
			}
		}
	}
	else if (const FDataLayerActorDescTreeItem* DataLayerActorDescTreeItem = Item.CastTo<FDataLayerActorDescTreeItem>())
	{
		if (UDataLayerInstance* DataLayerInstance = DataLayerActorDescTreeItem->GetDataLayer())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(DataLayerInstance))
			{
				return *ParentItem;
			}
			else if (bCreate)
			{
				return CreateDataLayerTreeItem(DataLayerInstance, true);
			}
		}
	}
	return nullptr;
}

void FDataLayerHierarchy::OnWorldPartitionCreated(UWorld* InWorld)
{
	if (RepresentingWorld.Get() == InWorld)
	{
		FullRefreshEvent();
	}
}

void FDataLayerHierarchy::OnLevelActorsAdded(const TArray<AActor*>& InActors)
{
	if (!bShowDataLayerActors)
	{
		return;
	}

	if (UWorld* OwningWorld = GetOwningWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;

		for (AActor* Actor : InActors)
		{
			if (Actor && (Actor->GetWorld() == OwningWorld))
			{
				TArray<const UDataLayerInstance*> DataLayerInstances = Actor->GetDataLayerInstances();
				EventData.Items.Reserve(DataLayerInstances.Num());
				for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
				{
					EventData.Items.Add(Mode->CreateItemFor<FDataLayerActorTreeItem>(FDataLayerActorTreeItemData(Actor, const_cast<UDataLayerInstance*>(DataLayerInstance))));
				}
			}
		}

		if (!EventData.Items.IsEmpty())
		{
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FDataLayerHierarchy::OnLevelActorsRemoved(const TArray<AActor*>& InActors)
{
	if (UWorld* OwningWorld = GetOwningWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;

		for (AActor* Actor : InActors)
		{
			if (Actor != nullptr)
			{
				const TArray<const UDataLayerInstance*> DataLayerInstances = Actor->GetDataLayerInstances();
				for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
				{
					EventData.ItemIDs.Add(FDataLayerActorTreeItem::ComputeTreeItemID(Actor, DataLayerInstance));
				}
			}
		}

		if (!EventData.ItemIDs.IsEmpty())
		{
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FDataLayerHierarchy::OnLevelActorAdded(AActor* InActor)
{
	OnLevelActorsAdded({ InActor });
}

void FDataLayerHierarchy::OnActorDataLayersChanged(const TWeakObjectPtr<AActor>& InActor)
{
	AActor* Actor = InActor.Get();
	if (Actor && (GetOwningWorld() == Actor->GetWorld()))
	{
		FullRefreshEvent();
	}
}

void FDataLayerHierarchy::OnDataLayerChanged(const EDataLayerAction Action, const TWeakObjectPtr<const UDataLayerInstance>& ChangedDataLayer, const FName& ChangedProperty)
{
	const UDataLayerInstance* DataLayerInstance = ChangedDataLayer.Get();
	if (DataLayerInstance || (Action == EDataLayerAction::Delete) || (Action == EDataLayerAction::Reset))
	{
		FullRefreshEvent();
	}
}

void FDataLayerHierarchy::OnLevelActorDeleted(AActor* InActor)
{
	OnLevelActorsRemoved({ InActor });
}

void FDataLayerHierarchy::OnLevelActorListChanged()
{
	FullRefreshEvent();
}

void FDataLayerHierarchy::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FDataLayerHierarchy::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FDataLayerHierarchy::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	if (!InWorld->IsGameWorld() && InLevel && (GetOwningWorld() == InWorld))
	{
		OnLevelActorsAdded(ObjectPtrDecay(InLevel->Actors));
	}
}

void FDataLayerHierarchy::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	if (!InWorld->IsGameWorld() && InLevel && (GetOwningWorld() == InWorld))
	{
		OnLevelActorsRemoved(ObjectPtrDecay(InLevel->Actors));
	}
}

void FDataLayerHierarchy::OnLoadedActorAdded(AActor& InActor)
{
	if (!bShowDataLayerActors)
	{
		return;
	}

	// Handle the actor being added to the level
	OnLevelActorAdded(&InActor);

	// Remove corresponding actor desc items
	if ((GetOwningWorld() == InActor.GetWorld()))
	{
		const TArray<const UDataLayerInstance*> DataLayerInstances = InActor.GetDataLayerInstances();
		if (DataLayerInstances.Num() > 0)
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Reserve(DataLayerInstances.Num());
		
			if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(&InActor))
			{
				if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(InActor.GetActorGuid()))
				{
					for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
					{
						EventData.ItemIDs.Add(FDataLayerActorDescTreeItem::ComputeTreeItemID(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance(), DataLayerInstance));
					}
				}
			}

			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FDataLayerHierarchy::OnLoadedActorRemoved(AActor& InActor)
{
	// Handle the actor being removed from the level
	OnLevelActorDeleted(&InActor);

	// Add any corresponding actor desc items for this actor
	UWorldPartition* const WorldPartition = FWorldPartitionHelpers::GetWorldPartition(&InActor);
	if ((GetOwningWorld() == InActor.GetWorld()) && WorldPartition != nullptr)
	{
		const TArray<const UDataLayerInstance*> DataLayerInstances = InActor.GetDataLayerInstances();
		if (DataLayerInstances.Num() > 0)
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
			EventData.Items.Reserve(DataLayerInstances.Num());
			for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
			{
				if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(InActor.GetActorGuid()); FActorDescTreeItem::ShouldDisplayInOutliner(ActorDescInstance))
				{
					EventData.Items.Add(Mode->CreateItemFor<FDataLayerActorDescTreeItem>(FDataLayerActorDescTreeItemData(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance(), const_cast<UDataLayerInstance*>(DataLayerInstance))));
				}
			}
			HierarchyChangedEvent.Broadcast(EventData);
		}
    }
}

void FDataLayerHierarchy::OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (!bShowUnloadedActors || !InActorDescInstance || InActorDescInstance->IsLoaded(true) || !FActorDescTreeItem::ShouldDisplayInOutliner(InActorDescInstance))
	{
		return;
	}

	const UDataLayerManager* const DataLayerManager = UDataLayerManager::GetDataLayerManager(GetOwningWorld());
	const FDataLayerInstanceNames& DataLayerInstanceNames = InActorDescInstance->GetDataLayerInstanceNames();

	if (DataLayerManager && DataLayerInstanceNames.Num() > 0)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;

		EventData.Items.Reserve(DataLayerInstanceNames.Num());
		for (const FName& DataLayerInstanceName : DataLayerInstanceNames.ToArray())
		{
			const UDataLayerInstance* const DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerInstanceName);
			EventData.Items.Add(Mode->CreateItemFor<FDataLayerActorDescTreeItem>(FDataLayerActorDescTreeItemData(InActorDescInstance->GetGuid(), InActorDescInstance->GetContainerInstance(), const_cast<UDataLayerInstance*>(DataLayerInstance))));
		}
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FDataLayerHierarchy::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (!bShowUnloadedActors || (InActorDescInstance == nullptr))
	{
		return;
	}

	const UDataLayerManager* const DataLayerManager = UDataLayerManager::GetDataLayerManager(GetOwningWorld());
	const FDataLayerInstanceNames& DataLayerInstanceNames = InActorDescInstance->GetDataLayerInstanceNames();
	
	if (DataLayerManager && DataLayerInstanceNames.Num() > 0)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Reserve(DataLayerInstanceNames.Num());

		for (const FName& DataLayerInstanceName : DataLayerInstanceNames.ToArray())
		{
			const UDataLayerInstance* const DataLayer = DataLayerManager->GetDataLayerInstance(DataLayerInstanceName);
			EventData.ItemIDs.Add(FDataLayerActorDescTreeItem::ComputeTreeItemID(InActorDescInstance->GetGuid(), InActorDescInstance->GetContainerInstance(), DataLayer));
		}
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FDataLayerHierarchy::FullRefreshEvent()
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;
	HierarchyChangedEvent.Broadcast(EventData);
}
