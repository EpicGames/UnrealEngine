// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DataLayer/DataLayerAction.h"
#include "ISceneOutlinerHierarchy.h"
#include "ISceneOutlinerTreeItem.h"
#include "SceneOutlinerFwd.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FDataLayerMode;
class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;
class UDataLayerInstance;
class ULevel;
class UWorld;
class UWorldPartition;
struct FSceneOutlinerTreeItemID;

class FDataLayerHierarchy : public ISceneOutlinerHierarchy
{
public:
	virtual ~FDataLayerHierarchy();
	static TUniquePtr<FDataLayerHierarchy> Create(FDataLayerMode* Mode, const TWeakObjectPtr<UWorld>& World);
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override {}
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
	void SetShowEditorDataLayers(bool bInShowEditorDataLayers) { bShowEditorDataLayers = bInShowEditorDataLayers; }
	void SetShowRuntimeDataLayers(bool bInShowRuntimeDataLayers) { bShowRuntimeDataLayers = bInShowRuntimeDataLayers; }
	void SetShowDataLayerActors(bool bInShowDataLayerActors) { bShowDataLayerActors = bInShowDataLayerActors; }
	void SetShowUnloadedActors(bool bInShowUnloadedActors) { bShowUnloadedActors = bInShowUnloadedActors; }
	void SetShowOnlySelectedActors(bool bInbShowOnlySelectedActors) { bShowOnlySelectedActors = bInbShowOnlySelectedActors; }
	void SetHighlightSelectedDataLayers(bool bInHighlightSelectedDataLayers) { bHighlightSelectedDataLayers = bInHighlightSelectedDataLayers; }
	void SetShowLevelInstanceContent(bool bInShowLevelInstanceContent) { bShowLevelInstanceContent = bInShowLevelInstanceContent; }
	bool GetShowLevelInstanceContent() const { return bShowLevelInstanceContent; }
private:
	UWorld* GetOwningWorld() const;
	FDataLayerHierarchy(FDataLayerMode* Mode, const TWeakObjectPtr<UWorld>& Worlds);
	FDataLayerHierarchy(const FDataLayerHierarchy&) = delete;
	FDataLayerHierarchy& operator=(const FDataLayerHierarchy&) = delete;

	void OnWorldPartitionCreated(UWorld* InWorld);
	void OnLevelActorsAdded(const TArray<AActor*>& InActors);
	void OnLevelActorsRemoved(const TArray<AActor*>& InActors);
	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorDeleted(AActor* InActor);
	void OnLevelActorListChanged();
	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);
	void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);
	void OnLoadedActorAdded(AActor& InActor);
	void OnLoadedActorRemoved(AActor& InActor);
	void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance);
	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance);
	void OnActorDataLayersChanged(const TWeakObjectPtr<AActor>& InActor);
	void OnDataLayerChanged(const EDataLayerAction Action, const TWeakObjectPtr<const UDataLayerInstance>& ChangedDataLayer, const FName& ChangedProperty);
	void FullRefreshEvent();
	FSceneOutlinerTreeItemPtr CreateDataLayerTreeItem(UDataLayerInstance* InDataLayer, bool bInForce = false) const;
	bool IsDataLayerPartOfSelection(const UDataLayerInstance* DataLayer) const;

	TWeakObjectPtr<UWorld> RepresentingWorld;
	bool bShowEditorDataLayers;
	bool bShowRuntimeDataLayers;
	bool bShowDataLayerActors;
	bool bShowUnloadedActors;
	bool bShowOnlySelectedActors;
	bool bHighlightSelectedDataLayers;
	bool bShowLevelInstanceContent;
};