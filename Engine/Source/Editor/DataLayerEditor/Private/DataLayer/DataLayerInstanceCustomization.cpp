// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerInstanceCustomization.h"

#include "Containers/Array.h"
#include "DetailLayoutBuilder.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "Templates/SharedPointer.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "FDataLayerInstanceDetails"

TSharedRef<IDetailCustomization> FDataLayerInstanceDetails::MakeInstance()
{
	return MakeShareable(new FDataLayerInstanceDetails);
}

void FDataLayerInstanceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	
	uint32 CustomizedDataLayerInstanceWithAssetCount = 0;
	bool bHasInitialRuntimeState = false;
	for (const TWeakObjectPtr<UObject>& SelectedObject : ObjectsBeingCustomized)
	{
		UDataLayerInstance* DataLayerInstance = Cast<UDataLayerInstance>(SelectedObject.Get());
		if (DataLayerInstance && DataLayerInstance->IsRuntime())
		{
			bHasInitialRuntimeState = true;
		}
		if (Cast<UDataLayerInstanceWithAsset>(SelectedObject.Get()))
		{
			++CustomizedDataLayerInstanceWithAssetCount;
		}
	}
	if (!bHasInitialRuntimeState)
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataLayerInstance, InitialRuntimeState));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataLayerInstance, OverrideBlockOnSlowStreaming));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataLayerInstance, StreamingPriority));
	}

	TSharedRef<IPropertyHandle> DataLayerAssetProperty = DetailBuilder.GetProperty("DataLayerAsset", UDataLayerInstanceWithAsset::StaticClass());
	UDataLayerInstanceWithAsset* CustomizedDataLayerInstanceWithAsset = (ObjectsBeingCustomized.Num() == 1) ? Cast<UDataLayerInstanceWithAsset>(ObjectsBeingCustomized[0].Get()) : nullptr;
	if (AWorldDataLayers* CustomizedWorldDataLayers = CustomizedDataLayerInstanceWithAsset ? CustomizedDataLayerInstanceWithAsset->GetDirectOuterWorldDataLayers() : nullptr)
	{
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Data Layer");
		Category.AddCustomRow(DataLayerAssetProperty->GetPropertyDisplayName())
			.RowTag(DataLayerAssetProperty->GetProperty()->GetFName())
			.NameContent()
			[
				DataLayerAssetProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(200.f)
			[
				SNew(SObjectPropertyEntryBox)
				.AllowClear(false)
				.AllowCreate(true)
				.AllowedClass(UDataLayerAsset::StaticClass())
				.PropertyHandle(DataLayerAssetProperty)
				.DisplayThumbnail(true)
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.OnShouldFilterAsset_Lambda([CustomizedWorldDataLayers](const FAssetData& AssetData)
				{
					FText FailureReason;
					const UDataLayerAsset* DataLayerAsset = CastChecked<UDataLayerAsset>(AssetData.GetAsset());
					return !CustomizedWorldDataLayers->CanReferenceDataLayerAsset(DataLayerAsset, &FailureReason);
				})
			];

		DetailBuilder.HideProperty(DataLayerAssetProperty);
	}
	else if (CustomizedDataLayerInstanceWithAssetCount > 1)
	{
		// We don't want to be able to set the same DataLayerAsset on multiple data layer instances
		if (IDetailPropertyRow* DetailPropertyRow = DetailBuilder.EditDefaultProperty(DataLayerAssetProperty))
		{
			DetailPropertyRow->IsEnabled(false);
		}
	}
}

#undef LOCTEXT_NAMESPACE
