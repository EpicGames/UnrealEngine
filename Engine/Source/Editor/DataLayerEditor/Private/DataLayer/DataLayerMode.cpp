// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerMode.h"

#include "ActorDescTreeItem.h"
#include "ActorMode.h"
#include "Algo/AnyOf.h"
#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "Algo/Accumulate.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetSelection.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/SparseArray.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "DataLayer/DataLayerDragDropOp.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayer/DataLayerOutlinerDeleteButtonColumn.h"
#include "DataLayer/SDataLayerOutliner.h"
#include "DataLayerActorTreeItem.h"
#include "DataLayerHierarchy.h"
#include "DataLayerTreeItem.h"
#include "DataLayerEditorModule.h"
#include "DataLayersActorDescTreeItem.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorActorFolders.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IContentBrowserSingleton.h"
#include "ISceneOutlinerHierarchy.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "SDataLayerBrowser.h"
#include "SSceneOutliner.h"
#include "SceneOutlinerFilters.h"
#include "SceneOutlinerMenuContext.h"
#include "SceneOutlinerPublicTypes.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Settings/EditorExperimentalSettings.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"
#include "FolderTreeItem.h"
#include "WorldDataLayersTreeItem.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldTreeItem.h"

class FWorldPartitionActorDesc;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "DataLayer"

using FDataLayerFilter = TSceneOutlinerPredicateFilter<FDataLayerTreeItem>;
using FDataLayerActorFilter = TSceneOutlinerPredicateFilter<FDataLayerActorTreeItem>;
using FActorDescFilter = TSceneOutlinerPredicateFilter<FActorDescTreeItem>;

FDataLayerModeParams::FDataLayerModeParams(SSceneOutliner* InSceneOutliner, SDataLayerBrowser* InDataLayerBrowser, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay, FOnSceneOutlinerItemPicked InOnItemPicked)
: SpecifiedWorldToDisplay(InSpecifiedWorldToDisplay)
, DataLayerBrowser(InDataLayerBrowser)
, SceneOutliner(InSceneOutliner)
, OnItemPicked(InOnItemPicked)
{}

FDataLayerMode::FDataLayerMode(const FDataLayerModeParams& Params)
	: ISceneOutlinerMode(Params.SceneOutliner)
	, OnItemPicked(Params.OnItemPicked)
	, DataLayerBrowser(Params.DataLayerBrowser)
	, SpecifiedWorldToDisplay(Params.SpecifiedWorldToDisplay)
	, FilteredDataLayerCount(0)
{

	Commands = MakeShareable(new FUICommandList());
	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateRaw(this, &FDataLayerMode::FindInContentBrowser),
		FCanExecuteAction::CreateRaw(this, &FDataLayerMode::CanFindInContentBrowser)
	));

	USelection::SelectionChangedEvent.AddRaw(this, &FDataLayerMode::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddRaw(this, &FDataLayerMode::OnLevelSelectionChanged);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FDataLayerMode::OnPostLoadMapWithWorld);

	UWorldPartitionEditorPerProjectUserSettings* SharedSettings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
	bHideEditorDataLayers = SharedSettings->bHideEditorDataLayers;
	bHideRuntimeDataLayers = SharedSettings->bHideRuntimeDataLayers;
	bHideDataLayerActors = SharedSettings->bHideDataLayerActors;
	bHideUnloadedActors = SharedSettings->bHideUnloadedActors;
	bShowOnlySelectedActors = SharedSettings->bShowOnlySelectedActors;
	bHighlightSelectedDataLayers = SharedSettings->bHighlightSelectedDataLayers;
	bHideLevelInstanceContent = SharedSettings->bHideLevelInstanceContent;

	FSceneOutlinerFilterInfo ShowOnlySelectedActorsInfo(LOCTEXT("ToggleShowOnlySelected", "Only Selected"), LOCTEXT("ToggleShowOnlySelectedToolTip", "When enabled, only displays actors that are currently selected."), bShowOnlySelectedActors, FCreateSceneOutlinerFilter::CreateStatic(&FDataLayerMode::CreateShowOnlySelectedActorsFilter));
	ShowOnlySelectedActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
	{
		UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
		Settings->bShowOnlySelectedActors = bShowOnlySelectedActors = bIsActive;
		Settings->PostEditChange();

		if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
		{
			DataLayerHierarchy->SetShowOnlySelectedActors(bIsActive);
		}

		RefreshSelection();
	});
	FilterInfoMap.Add(TEXT("ShowOnlySelectedActors"), ShowOnlySelectedActorsInfo);

	FSceneOutlinerFilterInfo HideEditorDataLayersInfo(LOCTEXT("ToggleHideEditorDataLayers", "Hide Editor Data Layers"), LOCTEXT("ToggleHideEditorDataLayersToolTip", "When enabled, hides Editor Data Layers."), bHideEditorDataLayers, FCreateSceneOutlinerFilter::CreateStatic(&FDataLayerMode::CreateHideEditorDataLayersFilter));
	HideEditorDataLayersInfo.OnToggle().AddLambda([this](bool bIsActive)
	{
		UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
		Settings->bHideEditorDataLayers = bHideEditorDataLayers = bIsActive;
		Settings->PostEditChange();

		if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
		{
			DataLayerHierarchy->SetShowEditorDataLayers(!bIsActive);
		}
	});
	FilterInfoMap.Add(TEXT("HideEditorDataLayersFilter"), HideEditorDataLayersInfo);

	FSceneOutlinerFilterInfo HideRuntimeDataLayersInfo(LOCTEXT("ToggleHideRuntimeDataLayers", "Hide Runtime Data Layers"), LOCTEXT("ToggleHideRuntimeDataLayersToolTip", "When enabled, hides Runtime Data Layers."), bHideRuntimeDataLayers, FCreateSceneOutlinerFilter::CreateStatic(&FDataLayerMode::CreateHideRuntimeDataLayersFilter));
	HideRuntimeDataLayersInfo.OnToggle().AddLambda([this](bool bIsActive)
	{
		UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
		Settings->bHideRuntimeDataLayers = bHideRuntimeDataLayers = bIsActive;
		Settings->PostEditChange();

		if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
		{
			DataLayerHierarchy->SetShowRuntimeDataLayers(!bIsActive);
		}
	});
	FilterInfoMap.Add(TEXT("HideRuntimeDataLayersFilter"), HideRuntimeDataLayersInfo);

	FSceneOutlinerFilterInfo HideDataLayerActorsInfo(LOCTEXT("ToggleHideDataLayerActors", "Hide Actors"), LOCTEXT("ToggleHideDataLayerActorsToolTip", "When enabled, hides Data Layer Actors."), bHideDataLayerActors, FCreateSceneOutlinerFilter::CreateStatic(&FDataLayerMode::CreateHideDataLayerActorsFilter));
	HideDataLayerActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
	{
		UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
		Settings->bHideDataLayerActors = bHideDataLayerActors = bIsActive;
		Settings->PostEditChange();

		if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
		{
			DataLayerHierarchy->SetShowDataLayerActors(!bIsActive);
		}
	});
	FilterInfoMap.Add(TEXT("HideDataLayerActorsFilter"), HideDataLayerActorsInfo);

	FSceneOutlinerFilterInfo HideUnloadedActorsInfo(LOCTEXT("ToggleHideUnloadedActors", "Hide Unloaded Actors"), LOCTEXT("ToggleHideUnloadedActorsToolTip", "When enabled, hides all unloaded world partition actors."), bHideUnloadedActors, FCreateSceneOutlinerFilter::CreateStatic(&FDataLayerMode::CreateHideUnloadedActorsFilter));
	HideUnloadedActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
	{
		UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
		Settings->bHideUnloadedActors = bHideUnloadedActors = bIsActive;
		Settings->PostEditChange();

		if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
		{
			DataLayerHierarchy->SetShowUnloadedActors(!bIsActive);
		}
	});
	FilterInfoMap.Add(TEXT("HideUnloadedActorsFilter"), HideUnloadedActorsInfo);

		
	// Add a an actor filter and interactive filter which sets the interactive mode of LevelInstance items and their children
	SceneOutliner->AddFilter(MakeShared<FDataLayerActorFilter>(
		FDataLayerActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* Actor, const UDataLayerInstance* DataLayer) 
		{
			return FActorMode::IsActorDisplayable(SceneOutliner, Actor, !bHideLevelInstanceContent);
		}), 
		FSceneOutlinerFilter::EDefaultBehaviour::Pass, 
		FDataLayerActorTreeItem::FInteractivePredicate::CreateLambda([this](const AActor* Actor, const UDataLayerInstance* DataLayer)
		{
			if (!bHideLevelInstanceContent)
			{
				if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(RepresentingWorld.Get()))
				{
					const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor);
					if (ParentLevelInstance && !LevelInstanceSubsystem->IsEditingLevelInstance(ParentLevelInstance))
					{
						return false;
					}
				}
			}
			return true;
		})));

	DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get();
	Rebuild();
	const_cast<FSharedSceneOutlinerData&>(SceneOutliner->GetSharedData()).CustomDelete = FCustomSceneOutlinerDeleteDelegate::CreateRaw(this, &FDataLayerMode::DeleteItems);
}

FDataLayerMode::~FDataLayerMode()
{
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
}

TSharedRef<FSceneOutlinerFilter> FDataLayerMode::CreateHideEditorDataLayersFilter()
{
	return MakeShareable(new FDataLayerFilter(FDataLayerTreeItem::FFilterPredicate::CreateStatic([](const UDataLayerInstance* DataLayerInstance) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FDataLayerMode::CreateHideRuntimeDataLayersFilter()
{
	return MakeShareable(new FDataLayerFilter(FDataLayerTreeItem::FFilterPredicate::CreateStatic([](const UDataLayerInstance* DataLayerInstance) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FDataLayerMode::CreateHideDataLayerActorsFilter()
{
	return MakeShareable(new FDataLayerActorFilter(FDataLayerActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* Actor, const UDataLayerInstance* DataLayerInstance) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FDataLayerMode::CreateHideUnloadedActorsFilter()
{
	return MakeShareable(new FActorDescFilter(FActorDescTreeItem::FFilterPredicate::CreateStatic([](const FWorldPartitionActorDescInstance* ActorDescInstance) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

int32 FDataLayerMode::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.IsA<FWorldDataLayersTreeItem>())
	{
		FWorldDataLayersTreeItem* WorldDataLayersTreeItem = (FWorldDataLayersTreeItem*)&Item;
		return static_cast<int32>(EItemSortOrder::WorldDataLayers) + WorldDataLayersTreeItem->GetSortPriority();
	}
	else if (Item.IsA<FDataLayerTreeItem>())
	{
		return static_cast<int32>(EItemSortOrder::DataLayer);
	}
	else if (Item.IsA<FDataLayerActorTreeItem>() || Item.IsA<FDataLayerActorDescTreeItem>())
	{
		return static_cast<int32>(EItemSortOrder::Actor);
	}
	// Warning: using actor mode with an unsupported item type!
	check(false);
	return -1;
}

bool FDataLayerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const 
{
	if (Item.IsValid() && (Item.IsA<FDataLayerTreeItem>()))
	{
		FDataLayerTreeItem* DataLayerTreeItem = (FDataLayerTreeItem*)&Item;
		return !DataLayerTreeItem->GetDataLayer()->IsReadOnly() && DataLayerTreeItem->GetDataLayer()->CanEditDataLayerShortName();
	}
	return false;
}

FText FDataLayerMode::GetStatusText() const
{
	// The number of DataLayers in the outliner before applying the text filter
	const int32 TotalDataLayerCount = ApplicableDataLayers.Num();
	const int32 SelectedDataLayerCount = SceneOutliner->GetSelection().Num<FDataLayerTreeItem>();

	if (!SceneOutliner->IsTextFilterActive())
	{
		if (SelectedDataLayerCount == 0)
		{
			return FText::Format(LOCTEXT("ShowingAllDataLayersFmt", "{0} data layers"), FText::AsNumber(FilteredDataLayerCount));
		}
		else
		{
			return FText::Format(LOCTEXT("ShowingAllDataLayersSelectedFmt", "{0} data layers ({1} selected)"), FText::AsNumber(FilteredDataLayerCount), FText::AsNumber(SelectedDataLayerCount));
		}
	}
	else if (SceneOutliner->IsTextFilterActive() && FilteredDataLayerCount == 0)
	{
		return FText::Format(LOCTEXT("ShowingNoDataLayersFmt", "No matching data layers ({0} total)"), FText::AsNumber(TotalDataLayerCount));
	}
	else if (SelectedDataLayerCount != 0)
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeDataLayersSelectedFmt", "Showing {0} of {1} data layers ({2} selected)"), FText::AsNumber(FilteredDataLayerCount), FText::AsNumber(TotalDataLayerCount), FText::AsNumber(SelectedDataLayerCount));
	}
	else
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeDataLayersFmt", "Showing {0} of {1} data layers"), FText::AsNumber(FilteredDataLayerCount), FText::AsNumber(TotalDataLayerCount));
	}
}

FFolder::FRootObject FDataLayerMode::GetRootObject() const
{
	return FFolder::GetWorldRootFolder(RepresentingWorld.Get()).GetRootObject();
}

SDataLayerBrowser* FDataLayerMode::GetDataLayerBrowser() const
{
	return DataLayerBrowser;
}

void FDataLayerMode::OnItemAdded(FSceneOutlinerTreeItemPtr Item)
{
	if (FDataLayerTreeItem* DataLayerItem = Item->CastTo<FDataLayerTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			++FilteredDataLayerCount;

			if (ShouldExpandDataLayer(DataLayerItem->GetDataLayer()))
			{
				SceneOutliner->SetItemExpansion(DataLayerItem->AsShared(), true);
			}

			if (SelectedDataLayersSet.Contains(DataLayerItem->GetDataLayer()))
			{
				SceneOutliner->AddToSelection({Item});
			}
		}
	}
	else if (const FDataLayerActorTreeItem* DataLayerActorTreeItem = Item->CastTo<FDataLayerActorTreeItem>())
	{
		if (SelectedDataLayerActors.Contains(FSelectedDataLayerActor(DataLayerActorTreeItem->GetDataLayer(), DataLayerActorTreeItem->GetActor())))
		{
			SceneOutliner->AddToSelection({Item});
		}
	}
}

void FDataLayerMode::OnItemRemoved(FSceneOutlinerTreeItemPtr Item)
{
	if (const FDataLayerTreeItem* DataLayerItem = Item->CastTo<FDataLayerTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			--FilteredDataLayerCount;
		}
	}
}

void FDataLayerMode::OnItemPassesFilters(const ISceneOutlinerTreeItem& Item)
{
	if (const FDataLayerTreeItem* const DataLayerItem = Item.CastTo<FDataLayerTreeItem>())
	{
		ApplicableDataLayers.Add(DataLayerItem->GetDataLayer());
	}
}

void FDataLayerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (FDataLayerTreeItem* DataLayerItem = Item->CastTo<FDataLayerTreeItem>())
	{
		if (UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer())
		{
			if (DataLayerInstance->CanBeInActorEditorContext())
			{
				if (!DataLayerInstance->IsInActorEditorContext())
				{
					const FScopedTransaction Transaction(LOCTEXT("MakeCurrentDataLayers", "Make Current Data Layer(s)"));

					if (!FSlateApplication::Get().GetModifierKeys().IsControlDown())
					{
						const UDataLayerManager* DataLayerManger = UDataLayerManager::GetDataLayerManager(GetOwningWorld());
						const TArray<UDataLayerInstance*> ActorEditorContextDataLayers = DataLayerManger->GetActorEditorContextDataLayers();

						for (UDataLayerInstance* DataLayerInstanceIt : ActorEditorContextDataLayers)
						{
							UDataLayerEditorSubsystem::Get()->RemoveFromActorEditorContext(DataLayerInstanceIt);
						}
					}

					UDataLayerEditorSubsystem::Get()->AddToActorEditorContext(DataLayerInstance);
				}
				else
				{
					const FScopedTransaction Transaction(LOCTEXT("RemoveCurrentDataLayers", "Remove Current Data Layer(s)"));

					if (!FSlateApplication::Get().GetModifierKeys().IsControlDown())
					{
						const UDataLayerManager* DataLayerManger = UDataLayerManager::GetDataLayerManager(GetOwningWorld());
						const TArray<UDataLayerInstance*> ActorEditorContextDataLayers = DataLayerManger->GetActorEditorContextDataLayers();
						for (UDataLayerInstance* DataLayerInstanceIt : ActorEditorContextDataLayers)
						{
							UDataLayerEditorSubsystem::Get()->RemoveFromActorEditorContext(DataLayerInstanceIt);
						}
					}
					else
					{
						UDataLayerEditorSubsystem::Get()->RemoveFromActorEditorContext(DataLayerInstance);
					}
				}
			}
		}
	}
	else if (FDataLayerActorTreeItem* DataLayerActorItem = Item->CastTo<FDataLayerActorTreeItem>())
	{
		if (AActor * Actor = DataLayerActorItem->GetActor())
		{
			const FScopedTransaction Transaction(LOCTEXT("ClickingOnActor", "Clicking on Actor in Data Layer"));
			GEditor->GetSelectedActors()->Modify();
			GEditor->SelectNone(/*bNoteSelectionChange*/false, true);
			GEditor->SelectActor(Actor, /*bSelected*/true, /*bNotify*/true, /*bSelectEvenIfHidden*/true);
			GEditor->NoteSelectionChange();
			GEditor->MoveViewportCamerasToActor(*Actor, /*bActiveViewportOnly*/false);
		}
	}
	else if (FDataLayerActorDescTreeItem* DataLayerActorDescItem = Item->CastTo<FDataLayerActorDescTreeItem>())
	{
		GEditor->BroadcastSelectUnloadedActors({ DataLayerActorDescItem->GetGuid() });
	}
}

void FDataLayerMode::DeleteItems(const TArray<TWeakPtr<ISceneOutlinerTreeItem>>& Items)
{
	TArray<UDataLayerInstance*> DataLayersToDelete;
	TMap<UDataLayerInstance*, TArray<AActor*>> ActorsToRemoveFromDataLayer;

	for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : Items)
	{
		if (FDataLayerActorTreeItem* DataLayerActorItem = Item.Pin()->CastTo<FDataLayerActorTreeItem>())
		{
			UDataLayerInstance* DataLayerInstance = DataLayerActorItem->GetDataLayer();
			AActor* Actor = DataLayerActorItem->GetActor();
			if (DataLayerInstance && !DataLayerInstance->IsReadOnly() && Actor)
			{
				ActorsToRemoveFromDataLayer.FindOrAdd(DataLayerInstance).Add(Actor);
			}
		}
		else if (FDataLayerTreeItem* DataLayerItem = Item.Pin()->CastTo< FDataLayerTreeItem>())
		{
			if (UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer())
			{
				if (!DataLayerInstance->IsReadOnly())
				{
					DataLayersToDelete.Add(DataLayerInstance);
				}
			}
		}
	}

	if (!ActorsToRemoveFromDataLayer.IsEmpty())
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveActorsFromDataLayer", "Remove Actors from Data Layer"));
		for (const auto& ItPair : ActorsToRemoveFromDataLayer)
		{
			DataLayerEditorSubsystem->RemoveActorsFromDataLayer(ItPair.Value, ItPair.Key);
		}
	}
	else if (!DataLayersToDelete.IsEmpty())
	{
		DeleteDataLayers(DataLayersToDelete);
	}
}

void FDataLayerMode::DeleteDataLayers(const TArray<UDataLayerInstance*>& InDataLayersToDelete)
{
	TArray<UDataLayerInstance*> DataLayersToDelete;
	for (UDataLayerInstance* DataLayerInstance : InDataLayersToDelete)
	{
		if (DataLayerInstance->CanBeRemoved())
		{
			DataLayersToDelete.Add(DataLayerInstance);
		}
	}

	int32 PrevDeleteCount = SelectedDataLayersSet.Num();
	for (UDataLayerInstance* DataLayerToDelete : DataLayersToDelete)
	{
		SelectedDataLayersSet.Remove(DataLayerToDelete);
	}

	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteSelectedDataLayers", "Delete Selected Data Layer(s)"));
		DataLayerEditorSubsystem->DeleteDataLayers(DataLayersToDelete);
	}

	if ((SelectedDataLayersSet.Num() != PrevDeleteCount) && DataLayerBrowser)
	{
		DataLayerBrowser->OnSelectionChanged(SelectedDataLayersSet);
	}
}

FReply FDataLayerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();

	// Rename key: Rename selected data layer (not rebindable, because it doesn't make much sense to bind.)
	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		if (Selection.Num() == 1)
		{
			FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
			if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
			{
				SceneOutliner->SetPendingRenameItem(ItemToRename);
				SceneOutliner->ScrollItemIntoView(ItemToRename);
			}
			return FReply::Handled();
		}
	}

	// F5 forces a full refresh
	else if (InKeyEvent.GetKey() == EKeys::F5)
	{
		SceneOutliner->FullRefresh();
		return FReply::Handled();
	}

	// Delete/BackSpace keys delete selected actors
	else if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		DeleteItems(Selection.SelectedItems);
		return FReply::Handled();
	}

	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool FDataLayerMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	return !GetDataLayerActorPairsFromOperation(Operation).IsEmpty()
		|| !GetActorsFromOperation(Operation, true).IsEmpty()
		|| !GetDataLayerInstancesFromOperation(Operation, true).IsEmpty()
		|| !GetDataLayerAssetsFromOperation(Operation, true).IsEmpty();
}

FSceneOutlinerDragValidationInfo FDataLayerMode::ValidateActorDrop(const ISceneOutlinerTreeItem& DropTarget, TArray<AActor*> PayloadActors, bool bMoveOperation /*= false*/) const
{
	// Only support dropping actors on Data Layer Instances
	const FDataLayerTreeItem* DataLayerItem = DropTarget.CastTo<FDataLayerTreeItem>();
	const UDataLayerInstance* DropTargetDataLayerInstance = DataLayerItem ? GetDataLayerInstanceFromTreeItem(DropTarget) : nullptr;
	if (!DropTargetDataLayerInstance)
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
	}

	// Check if user can't add actors to the target Data Layer Instance
	{
		FText Reason;
		if (!DropTargetDataLayerInstance->CanUserAddActors(&Reason))
		{
			const FText Text = FText::Format(LOCTEXT("CantMoveOrAssignActorsToLockedDataLayer", "Can't {0} actor(s) to Data Layer \"{1}\" : {2}"), bMoveOperation ? FText::FromString(TEXT("move")) : FText::FromString(TEXT("assign")), FText::FromString(DropTargetDataLayerInstance->GetDataLayerShortName()), Reason);
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
		}
	}

	// Check if can't add one of the payload actors to the target Data Layer Instance
	for (AActor* Actor : PayloadActors)
	{
		FText Reason;
		if (!DropTargetDataLayerInstance->CanAddActor(Actor, &Reason))
		{
			const FText Text = FText::Format(LOCTEXT("CantMoveOrAssignActorsToDataLayer", "Can't {0} actor(s) to Data Layer \"{1}\" : {2}"), bMoveOperation ? FText::FromString(TEXT("move")) : FText::FromString(TEXT("assign")), FText::FromString(DropTargetDataLayerInstance->GetDataLayerShortName()), Reason);
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
		}
	}

	if (!bMoveOperation)
	{
		if (GetSelectedDataLayers(SceneOutliner).Num() > 1)
		{
			if (SceneOutliner->GetTree().IsItemSelected(const_cast<ISceneOutlinerTreeItem&>(DropTarget).AsShared()))
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, LOCTEXT("AssignToDataLayers", "Assign to Selected Data Layers"));
			}
		}
	}
	const FText Text = FText::Format(LOCTEXT("MoveOrAssignToDataLayer", "{0} to Data Layer \"{1}\""), bMoveOperation ? FText::FromString(TEXT("Move")) : FText::FromString(TEXT("Assign")), FText::FromString(DropTargetDataLayerInstance->GetDataLayerShortName()));
	return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, Text);
}

FSceneOutlinerDragValidationInfo FDataLayerMode::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	TArray<AActor*> PayloadActors = GetActorsFromOperation(Payload.SourceOperation);
	if (!PayloadActors.IsEmpty()) // Adding actor(s) in data layer(s)
	{
		const FDataLayerTreeItem* DataLayerItem = DropTarget.CastTo<FDataLayerTreeItem>();
		if (UDataLayerInstance* TargetDataLayerInstance = DataLayerItem ? DataLayerItem->GetDataLayer() : nullptr)
		{
			const bool bIsPrivateDataLayer = TargetDataLayerInstance->GetAsset() && TargetDataLayerInstance->GetAsset()->IsPrivate();
			const AWorldDataLayers* OuterWorldDataLayers = TargetDataLayerInstance->GetDirectOuterWorldDataLayers();

			for (AActor* Actor : PayloadActors)
			{
				if (!Actor->IsUserManaged() || !DataLayerEditorSubsystem->IsActorValidForDataLayerInstances(Actor, { TargetDataLayerInstance }))
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("ActorCantBeAssignedToDataLayer", "Can't assign actors to Data Layer"));
				}
				else if (bIsPrivateDataLayer && (Actor->GetLevel()->GetWorldDataLayers() != OuterWorldDataLayers))
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("ActorCantBeAssignedToPrivateDataLayerOfOtherWorldDataLayers", "Can't assign actors to Private Data Layer of another WorldDataLayers actor"));
				}
			}
		}

		return ValidateActorDrop(DropTarget, PayloadActors);
	}
	
	// Drag and drop actor(s) into a Data Layer Instance
	if (!GetDataLayerActorPairsFromOperation(Payload.SourceOperation).IsEmpty())
	{
		return ValidateActorDrop(DropTarget, PayloadActors, true);
	}
	
	// Drag and drop Data Layer Asset(s)
	{
		TArray<const UDataLayerAsset*> DataLayerAssets = GetDataLayerAssetsFromOperation(Payload.SourceOperation);
		if (!DataLayerAssets.IsEmpty())
		{
			return ValidateDataLayerAssetDrop(DropTarget, DataLayerAssets);
		}
	}
	
	// Drag and drop Data Layer Instance(s)
	TArray<UDataLayerInstance*> PayloadDataLayers = GetDataLayerInstancesFromOperation(Payload.SourceOperation);
	if (!PayloadDataLayers.IsEmpty())
	{
		// Drag and drop DataLayerInstance(s) from different AWorldDataLayers
		AWorldDataLayers* PayloadWorldDatalayers = nullptr;
		for (UDataLayerInstance* DataLayerInstance : PayloadDataLayers)
		{
			if (!PayloadWorldDatalayers)
			{
				PayloadWorldDatalayers = DataLayerInstance->GetDirectOuterWorldDataLayers();
			}
			else if (PayloadWorldDatalayers != DataLayerInstance->GetDirectOuterWorldDataLayers())
			{
				PayloadWorldDatalayers = nullptr;
				break;
			}
		}
		if (!PayloadWorldDatalayers)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantMoveMultipleDataLayersComingFromDifferentWorldDataLayers", "Can't move multiple Data Layers coming from different WorldDataLayers"));
		}

		// Drag and drop DataLayerInstanceWithAsset(s) between AWorldDataLayers
		AWorldDataLayers* DropTargetWorldDataLayers = GetWorldDataLayersFromTreeItem(DropTarget);
		if (DropTargetWorldDataLayers && (PayloadWorldDatalayers != DropTargetWorldDataLayers))
		{
			TSet<const UDataLayerAsset*> DataLayerAssets;
			for (UDataLayerInstance* DataLayerInstance : PayloadDataLayers)
			{
				if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
				{
					if (const UDataLayerAsset* DataLayerAsset = DataLayerInstanceWithAsset->GetAsset())
					{
						DataLayerAssets.Add(DataLayerAsset);
					}
				}
			}
			if (!DataLayerAssets.IsEmpty())
			{
				const bool bIsMovingDataLayerInstances = true;
				return ValidateDataLayerAssetDrop(DropTarget, DataLayerAssets.Array(), bIsMovingDataLayerInstances);
			}
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("NoValidDataLayersToMove", "No valid Data Layer to move"));
		}

		const UDataLayerInstance* ParentDataLayerInstance = GetDataLayerInstanceFromTreeItem(DropTarget);
		for (UDataLayerInstance* DataLayerInstance : PayloadDataLayers)
		{
			{
				FText Reason;
				if (DataLayerInstance->IsReadOnly(&Reason))
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText::Format(LOCTEXT("CantMoveReadOnlyDataLayer", "Can't move Data Layer : {0}"), Reason));
				}
			}

			if (ParentDataLayerInstance != nullptr)
			{
				FText Reason;
				if (!DataLayerInstance->CanBeChildOf(ParentDataLayerInstance, &Reason))
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Reason);
				}
				
				if (ParentDataLayerInstance->IsReadOnly(&Reason))
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText::Format(LOCTEXT("CantMoveDataLayerToLockedDataLayer", "Can't move to Data Layer : {0}"), Reason));
				}
			}
		}

		if (ParentDataLayerInstance)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, FText::Format(LOCTEXT("MoveDataLayerToDataLayer", "Move to Data Layer \"{0}\""), FText::FromString(ParentDataLayerInstance->GetDataLayerShortName())));
		}
		else
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, LOCTEXT("MoveDataLayerToRoot", "Move to root"));
		}	
	}

	return FSceneOutlinerDragValidationInfo::Invalid();
}

UDataLayerInstance* FDataLayerMode::GetDataLayerInstanceFromTreeItem(const ISceneOutlinerTreeItem& TreeItem) const
{
	const FDataLayerActorTreeItem* DataLayerActorTargetItem = TreeItem.CastTo<FDataLayerActorTreeItem>();
	const FDataLayerTreeItem* DataLayerTargetItem = TreeItem.CastTo<FDataLayerTreeItem>();
	UDataLayerInstance* DataLayerTarget = DataLayerTargetItem ? DataLayerTargetItem->GetDataLayer() : (DataLayerActorTargetItem ? DataLayerActorTargetItem->GetDataLayer() : nullptr);
	return DataLayerTarget;
}

AWorldDataLayers* FDataLayerMode::GetWorldDataLayersFromTreeItem(const ISceneOutlinerTreeItem& TreeItem) const
{
	AWorldDataLayers* WorldDataLayers = GetOwningWorldAWorldDataLayers();

	if (const FWorldDataLayersTreeItem* WorldDataLayerTreeItem = TreeItem.CastTo<FWorldDataLayersTreeItem>())
	{
		WorldDataLayers = WorldDataLayerTreeItem->GetWorldDataLayers();
	}
	else if (const FFolderTreeItem* FolderTreeItem = TreeItem.CastTo<FFolderTreeItem>())
	{
		if (const UObject* RootObject = FolderTreeItem->GetFolder().GetRootObjectPtr())
		{
			if (AWorldDataLayers* RootObjectWorldDataLayers = RootObject->GetWorld() ? RootObject->GetWorld()->GetWorldDataLayers() : nullptr)
			{
				WorldDataLayers = RootObjectWorldDataLayers;
			}
		}
	}
	else if (const UDataLayerInstance* DataLayerTarget = GetDataLayerInstanceFromTreeItem(TreeItem))
	{
		WorldDataLayers = DataLayerTarget->GetDirectOuterWorldDataLayers();
	}
	
	return WorldDataLayers;
}

bool FDataLayerMode::CanReferenceDataLayerAssets(const AWorldDataLayers* InWorldDataLayers, const TArray<const UDataLayerAsset*>& InReferencedDataLayerAssets, FText* OutFailureReason) const
{
	const AWorldDataLayers* ReferencingWorldDataLayers = InWorldDataLayers ? InWorldDataLayers : GetOwningWorldAWorldDataLayers();
	for (const UDataLayerAsset* DataLayerAsset : InReferencedDataLayerAssets)
	{
		if (!ReferencingWorldDataLayers->CanReferenceDataLayerAsset(DataLayerAsset, OutFailureReason))
		{
			return false;
		}
	}
	return true;
}

FSceneOutlinerDragValidationInfo FDataLayerMode::ValidateDataLayerAssetDrop(const ISceneOutlinerTreeItem& DropTarget, const TArray<const UDataLayerAsset*>& DataLayerAssetsToDrop, bool bIsMovingDataLayerInstances) const
{
	check(!DataLayerAssetsToDrop.IsEmpty());
	const FString CreateOrMoveString(bIsMovingDataLayerInstances ? "move" : "create");

	const AWorldDataLayers* DropTargetWorldDataLayers = GetWorldDataLayersFromTreeItem(DropTarget);
	const UDataLayerInstance* DropTargetDataLayerInstance = GetDataLayerInstanceFromTreeItem(DropTarget);
	const UDataLayerInstanceWithAsset* DropTargetDataLayerWithAsset = DropTargetDataLayerInstance ? Cast<UDataLayerInstanceWithAsset>(DropTargetDataLayerInstance) : nullptr;
	if (!DropTarget.CanInteract() || (DropTargetDataLayerWithAsset && DropTargetDataLayerWithAsset->IsReadOnly()))
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("DropTargetIsReadOnly", "\"{0}\" is read only."), FText::FromString(DropTarget.GetDisplayString())));
	}

	// Check for duplicate Data Layer Asset using Outer World's AWorldDataLayers
	UWorld* OuterWorld = DropTargetWorldDataLayers->GetTypedOuter<UWorld>();
	const AWorldDataLayers* OuterWorldDataLayers = OuterWorld->GetWorldDataLayers();
	const TArray<const UDataLayerInstance*> ExistingDataLayerInstances = OuterWorldDataLayers->GetDataLayerInstances(DataLayerAssetsToDrop);
	if (!ExistingDataLayerInstances.IsEmpty())
	{
		TArray<FString> ExistingAssestNames;
		Algo::Transform(ExistingDataLayerInstances, ExistingAssestNames, [](const UDataLayerInstance* DataLayerInstance) { return Cast<UDataLayerInstanceWithAsset>(DataLayerInstance)->GetAsset()->GetName(); });
		FString ExistingAssetNamesString = FString::Join(ExistingAssestNames, TEXT(","));
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("CantCreateDataLayerInstanceSameAsset", "Cannot create Data Layer Instance(s), there are already Data Layer Instance(s) for \"{0}\"."), FText::FromString(ExistingAssetNamesString)));
	}

	if (Algo::AnyOf(DataLayerAssetsToDrop, [](const UDataLayerAsset* DataLayerAsset) { return DataLayerAsset && DataLayerAsset->IsA<UExternalDataLayerAsset>(); }))
	{
		// Check if External Data Layer Asset can be added
		UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(OuterWorld);
		if (!ExternalDataLayerManager)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, LOCTEXT("CantCreateDataLayerInstanceWithExternalDataLayerAssetNotSupported", "Cannot create External Data Layer Instance : Partitioned world doesn't support External Data Layers."));
		}
		if (!GetDefault<UEditorExperimentalSettings>()->bEnableWorldPartitionExternalDataLayers)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, LOCTEXT("CantCreateExternalDataLayerInstanceExperimentalFlagDisabled", "Cannot create External Data Layer Instance : Experimental flag 'Enable World Partition External Data Layers' is disabled."));
		}
		FText InjectionFailureReason;
		if (Algo::AnyOf(DataLayerAssetsToDrop, [ExternalDataLayerManager, &InjectionFailureReason](const UDataLayerAsset* DataLayerAsset) { return DataLayerAsset && DataLayerAsset->IsA<UExternalDataLayerAsset>() && !ExternalDataLayerManager->CanInjectExternalDataLayerAsset(Cast<UExternalDataLayerAsset>(DataLayerAsset), &InjectionFailureReason); }))
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("CantCreateDataLayerInstanceWithExternalDataLayerAsset", "Cannot create External Data Layer Instance : {0}"), InjectionFailureReason));
		}

		if (DropTarget.IsA<FDataLayerTreeItem>() || DropTarget.IsA<FDataLayerActorTreeItem>())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("CantCreateDataLayerInstanceUnderItem", "Cannot create External Data Layer Instance under {0}"), FText::FromString(DropTarget.GetDisplayString())));
		}
	}

	// Special case where we receive external data layer asset(s)
	if (Algo::AllOf(DataLayerAssetsToDrop, [](const UDataLayerAsset* DataLayerAsset) { return DataLayerAsset && DataLayerAsset->IsA<UExternalDataLayerAsset>(); }))
	{
		// Only allow to create Data Layer Instance referencing an External Data Layer Asset if the target WorldDataLayers is the main one
		if (DropTargetWorldDataLayers == GetOwningWorldAWorldDataLayers())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, LOCTEXT("CreateAllExternalDataLayerFromAssetDrop", "Create External Data Layer Instances"));
		}
	}

	// Check if can reference Data Layer Asset
	FText FailureReason;
	if (!CanReferenceDataLayerAssets(DropTargetWorldDataLayers, DataLayerAssetsToDrop, &FailureReason))
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("CantCreateOrMoveDataLayerInstancePassFilterFailed", "Cannot {0} Data Layer Instance: {1}"), FText::FromString(CreateOrMoveString), FailureReason));
	}

	// Check if target data layer supports having dropped asset as a child
	if (DropTargetDataLayerWithAsset)
	{
		EDataLayerType ParentType = DropTargetDataLayerWithAsset->GetType();
		for (const UDataLayerAsset* DataLayerAssetToDrop : DataLayerAssetsToDrop)
		{
			FText Reason;
			if (!FDataLayerUtils::AreDataLayerTypesCompatible(ParentType, DataLayerAssetToDrop->GetType(), DropTargetDataLayerWithAsset->IsA<UExternalDataLayerInstance>(), &Reason))
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FText::Format(LOCTEXT("CantCreateOrMoveDataLayerInstanceIncompatibleChildType", "Cannot {0} Data Layer Instance: {1}"), FText::FromString(CreateOrMoveString), Reason));
			}
		}
	}
	
	if (DropTarget.CastTo<FDataLayerTreeItem>() || DropTarget.CastTo<FWorldDataLayersTreeItem>())
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, FText::Format(LOCTEXT("CreateAllDataLayerFromAssetDropOnPart", "Create Data Layer Instances under \"{0}\""), FText::FromString(DropTarget.GetDisplayString())));
	}
	else
	{
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Compatible, LOCTEXT("CreateAllDataLayerFromAssetDrop", "Create Data Layer Instances"));
	}
}

TArray<UDataLayerInstance*> FDataLayerMode::GetDataLayerInstancesFromOperation(const FDragDropOperation& Operation, bool bOnlyFindFirst /*= false*/) const
{
	TArray<UDataLayerInstance*> OutDataLayerInstances;
	
	auto GetDataLayers = [this, &OutDataLayerInstances](const FDataLayerDragDropOp& DataLayerOp)
	{
		for (const TWeakObjectPtr<UDataLayerInstance>& DragDropInfo : DataLayerOp.DataLayerInstances)
		{
			if (UDataLayerInstance* DataLayerInstance = DragDropInfo.Get())
			{
				OutDataLayerInstances.AddUnique(DataLayerInstance);
			}
		}
	};

	if (Operation.IsOfType<FCompositeDragDropOp>())
	{
		const FCompositeDragDropOp& CompositeDragDropOp = StaticCast<const FCompositeDragDropOp&>(Operation);
		if (TSharedPtr<const FDataLayerDragDropOp> DataLayerDragDropOp = CompositeDragDropOp.GetSubOp<FDataLayerDragDropOp>())
		{
			GetDataLayers(*DataLayerDragDropOp);
		}
	}
	else if (Operation.IsOfType<FDataLayerDragDropOp>())
	{
		const FDataLayerDragDropOp& DataLayerDragDropOp = StaticCast<const FDataLayerDragDropOp&>(Operation);
		GetDataLayers(DataLayerDragDropOp);
	}

	return OutDataLayerInstances;
}

TArray<const UDataLayerAsset*> FDataLayerMode::GetDataLayerAssetsFromOperation(const FDragDropOperation& InDragDrop, bool bOnlyFindFirst /* = false */) const
{
	TArray<const UDataLayerAsset*> OutDataLayerAssets;

	TArray<FAssetData> AssetDatas = AssetUtil::ExtractAssetDataFromDrag(InDragDrop.AsShared());
	for (const FAssetData& AssetData : AssetDatas)
	{
		if (AssetData.GetClass()->IsChildOf<UDataLayerAsset>())
		{
			if (UObject* Asset = AssetData.GetAsset())
			{
				OutDataLayerAssets.Add(StaticCast<UDataLayerAsset*>(Asset));
			}
		}
	}

	return OutDataLayerAssets;
}

TArray<FDataLayerActorMoveElement> FDataLayerMode::GetDataLayerActorPairsFromOperation(const FDragDropOperation& Operation) const
{
	TArray<FDataLayerActorMoveElement> Out;

	if (Operation.IsOfType<FDataLayerActorMoveOp>())
	{
		const FDataLayerActorMoveOp& DataLayerActorDragOp = StaticCast<const FDataLayerActorMoveOp&>(Operation);
		return DataLayerActorDragOp.DataLayerActorMoveElements;
	}

	return Out;
}

TArray<AActor*> FDataLayerMode::GetActorsFromOperation(const FDragDropOperation& Operation, bool bOnlyFindFirst /*= false*/ ) const
{
	TSet<AActor*> Actors;

	auto GetActorsFromFolderOperation = [&Actors, bOnlyFindFirst](const FFolderDragDropOp& FolderOp)
	{
		if (bOnlyFindFirst && Actors.Num())
		{
			return;
		}

		if (UWorld* World = FolderOp.World.Get())
		{
			TArray<TWeakObjectPtr<AActor>> ActorsToDrop;
			FActorFolders::GetWeakActorsFromFolders(*World, FolderOp.Folders, ActorsToDrop, FolderOp.RootObject);
			for (const auto& Actor : ActorsToDrop)
			{
				if (AActor* ActorPtr = Actor.Get())
				{
					Actors.Add(ActorPtr);
					if (bOnlyFindFirst)
					{
						break;
					}
				}
			}
		}
	};

	auto GetActorsFromActorOperation = [&Actors, bOnlyFindFirst](const FActorDragDropOp& ActorOp)
	{
		if (bOnlyFindFirst && Actors.Num())
		{
			return;
		}
		for (const auto& Actor : ActorOp.Actors)
		{
			if (AActor* ActorPtr = Actor.Get())
			{
				Actors.Add(ActorPtr);
				if (bOnlyFindFirst)
				{
					break;
				}
			}
		}
	};

	if (Operation.IsOfType<FActorDragDropOp>())
	{
		const FActorDragDropOp& ActorDragOp = StaticCast<const FActorDragDropOp&>(Operation);
		GetActorsFromActorOperation(ActorDragOp);
	}
	if (Operation.IsOfType<FFolderDragDropOp>())
	{
		const FFolderDragDropOp& FolderDragOp = StaticCast<const FFolderDragDropOp&>(Operation);
		GetActorsFromFolderOperation(FolderDragOp);
	}
	if (Operation.IsOfType<FCompositeDragDropOp>())
	{
		const FCompositeDragDropOp& CompositeDragOp = StaticCast<const FCompositeDragDropOp&>(Operation);
		if (TSharedPtr<const FActorDragDropOp> ActorSubOp = CompositeDragOp.GetSubOp<FActorDragDropOp>())
		{
			GetActorsFromActorOperation(*ActorSubOp);
		}
		if (TSharedPtr<const FFolderDragDropOp> FolderSubOp = CompositeDragOp.GetSubOp<FFolderDragDropOp>())
		{
			GetActorsFromFolderOperation(*FolderSubOp);
		}
	}
	return Actors.Array();
}

void FDataLayerMode::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	UDataLayerInstance* TargetDataLayerInstance = GetDataLayerInstanceFromTreeItem(DropTarget);

	TArray<AActor*> ActorsToAdd = GetActorsFromOperation(Payload.SourceOperation);
	if (!ActorsToAdd.IsEmpty()) // Adding actor(s) in data layer(s)
	{
		if (SceneOutliner->GetTree().IsItemSelected(const_cast<ISceneOutlinerTreeItem&>(DropTarget).AsShared()))
		{
			TArray<UDataLayerInstance*> AllSelectedDataLayers = GetSelectedDataLayers(SceneOutliner);
			if (AllSelectedDataLayers.Num() > 1)
			{
				const FScopedTransaction Transaction(LOCTEXT("DataLayerOutlinerAddActorsToDataLayers", "Add Actors to Data Layers"));
				DataLayerEditorSubsystem->AddActorsToDataLayers(ActorsToAdd, AllSelectedDataLayers);
				return;
			}
		}

		if (TargetDataLayerInstance)
		{
			const FScopedTransaction Transaction(LOCTEXT("DataLayerOutlinerAddActorsToDataLayer", "Add Actors to Data Layer"));
			DataLayerEditorSubsystem->AddActorsToDataLayer(ActorsToAdd, TargetDataLayerInstance);
		}

		return;
	}
	
	if (Payload.SourceOperation.IsOfType<FDataLayerActorMoveOp>()) // Moving actor(s) into a Data Layer
	{
		TArray<FDataLayerActorMoveElement> ActorsToMove = GetDataLayerActorPairsFromOperation(Payload.SourceOperation);
		if (!ActorsToMove.IsEmpty() && TargetDataLayerInstance)
		{
			const FScopedTransaction Transaction(LOCTEXT("DataLayerOutlinerMoveActorsToDataLayer", "Move Actors to Data Layer"));
			for (const auto& DataLayerActorPair : ActorsToMove)
			{
				if (AActor* ActorPtr = DataLayerActorPair.Key.Get())
				{
					DataLayerEditorSubsystem->AddActorToDataLayer(ActorPtr, TargetDataLayerInstance);
					DataLayerEditorSubsystem->RemoveActorFromDataLayer(ActorPtr, DataLayerActorPair.Value.Get());
				}
			}
		}

		return;
	}

	// Handle dropping Data Layer Asset(s)
	TArray<const UDataLayerAsset*> DataLayerAssetToDrop = GetDataLayerAssetsFromOperation(Payload.SourceOperation);
	if (!DataLayerAssetToDrop.IsEmpty())
	{
		OnDataLayerAssetDropped(DataLayerAssetToDrop, DropTarget);
		return;
	}

	// Handle dropping DataLayerInstance(s)
	TArray<UDataLayerInstance*> DataLayerInstances = GetDataLayerInstancesFromOperation(Payload.SourceOperation);

	// Handle dropping DataLayerInstanceWithAsset(s) in a different AWorldDataLayers
	AWorldDataLayers* DropTargetWorldDataLayers = GetWorldDataLayersFromTreeItem(DropTarget);
	AWorldDataLayers* PayloadWorldDatalayers = DataLayerInstances.Num() ? DataLayerInstances[0]->GetDirectOuterWorldDataLayers() : nullptr; // Validation guarantees that they are all part of the same AWorldDataLayers
	if (DropTargetWorldDataLayers && (PayloadWorldDatalayers != DropTargetWorldDataLayers))
	{
		TSet<const UDataLayerAsset*> DataLayerAssets;
		for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
			{
				DataLayerAssets.Add(DataLayerInstanceWithAsset->GetAsset());
			}
		}
		if (!DataLayerAssets.IsEmpty())
		{
			OnDataLayerAssetDropped(DataLayerAssets.Array(), DropTarget);
			return;
		}
	}
	
	// Moving DataLayerInstance(s)
	SetParentDataLayer(DataLayerInstances, TargetDataLayerInstance);
}

void FDataLayerMode::OnDataLayerAssetDropped(const TArray<const UDataLayerAsset*>& DroppedDataLayerAsset, ISceneOutlinerTreeItem& DropTarget) const
{
	const FScopedTransaction Transaction(LOCTEXT("DataLayerOutlinerCreateDataLayerInstanceFromAssetDrop", "Create Data Layer Instance from Data Layer Asset Drop"));

	for (const UDataLayerAsset* DataLayerAsset : DroppedDataLayerAsset)
	{
		FDataLayerCreationParameters Params;
		Params.DataLayerAsset = const_cast<UDataLayerAsset*>(DataLayerAsset);
		Params.WorldDataLayers = GetWorldDataLayersFromTreeItem(DropTarget);
		if (UDataLayerInstance* DataLayerInstance = DataLayerEditorSubsystem->CreateDataLayerInstance(Params))
		{
			if (UDataLayerInstance* ParentDataLayerInstance = GetDataLayerInstanceFromTreeItem(DropTarget))
			{
				DataLayerEditorSubsystem->SetParentDataLayer(DataLayerInstance, ParentDataLayerInstance);
			}
		}
	}
}

FReply FDataLayerMode::OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const
{
	TSharedPtr<FDragDropOperation> DragOperation = Event.GetOperation();
	
	if (!DragOperation.IsValid())
	{
		return FReply::Handled();
	}

	if (DragOperation->IsOfType<FDataLayerActorMoveOp>()) // Moving actor(s) into a Data Layer
	{
		DragOperation->SetCursorOverride(EMouseCursor::GrabHandClosed);
	}
	else if (DragOperation->IsOfType<FActorDragDropOp>()) // Adding actor(s) in data layer(s)
	{
		DragOperation->SetCursorOverride(EMouseCursor::Default);
	}

	return FReply::Handled();
}

void FDataLayerMode::SetParentDataLayer(const TArray<UDataLayerInstance*> DataLayerInstances, UDataLayerInstance* ParentDataLayer) const
{
	if (!DataLayerInstances.IsEmpty())
	{
		TArray<UDataLayerInstance*> ValidDataLayers;
		ValidDataLayers.Reserve(DataLayerInstances.Num());
		for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			if (DataLayerInstance->CanBeChildOf(ParentDataLayer))
			{
				ValidDataLayers.Add(DataLayerInstance);
			}
		}

		if (!ValidDataLayers.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("DataLayerOutlinerChangeDataLayersParent", "Change Data Layers Parent"));
			DataLayerEditorSubsystem->SetParentDataLayerForDataLayers(ValidDataLayers, ParentDataLayer);
		}
	}
}

struct FWeakDataLayerActorSelector
{
	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FDataLayerActorTreeItem* TypedItem = ItemPtr->CastTo<FDataLayerActorTreeItem>())
			{
				if (TypedItem->IsValid())
				{
					DataOut = TypedItem->Actor;
					return true;
				}
			}
		}
		return false;
	}
};

struct FDataLayerActorPairSelector
{
	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FDataLayerActorMoveElement& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FDataLayerActorTreeItem* TypedItem = ItemPtr->CastTo<FDataLayerActorTreeItem>())
			{
				if (TypedItem->IsValid())
				{
					DataOut = FDataLayerActorMoveElement(TypedItem->Actor.Get(), TypedItem->GetDataLayer());
					return true;
				}
			}
		}
		return false;
	}
};


struct FWeakDataLayerSelector
{
	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<UDataLayerInstance>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FDataLayerTreeItem* TypedItem = ItemPtr->CastTo<FDataLayerTreeItem>())
			{
				if (TypedItem->IsValid())
				{
					DataOut = TypedItem->GetDataLayer();
					return true;
				}
			}
		}
		return false;
	}
};

TSharedPtr<FDragDropOperation> FDataLayerMode::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	FSceneOutlinerDragDropPayload DraggedObjects(InTreeItems);

	if (DraggedObjects.Has<FDataLayerTreeItem>())
	{
		TArray<TWeakObjectPtr<UDataLayerInstance>> DataLayerInstances = DraggedObjects.GetData<TWeakObjectPtr<UDataLayerInstance>>(FWeakDataLayerSelector());
		if (DataLayerInstances.FindByPredicate([&](const TWeakObjectPtr<UDataLayerInstance>& DataLayerInstance) { return DataLayerInstance.IsValid() && DataLayerInstance->IsReadOnly(); }))
		{
			return TSharedPtr<FDragDropOperation>();
		}
	}

	auto GetDataLayerOperation = [&DraggedObjects]()
	{
		TSharedPtr<FDataLayerDragDropOp> DataLayerOperation = MakeShareable(new FDataLayerDragDropOp);
		TArray<TWeakObjectPtr<UDataLayerInstance>> DataLayers = DraggedObjects.GetData<TWeakObjectPtr<UDataLayerInstance>>(FWeakDataLayerSelector());
		DataLayerOperation->Init(DataLayers);
		DataLayerOperation->Construct();
		return DataLayerOperation;
	};

	auto GetActorOperation = [&DraggedObjects]()
	{
		TSharedPtr<FActorDragDropOp> ActorOperation = MakeShareable(new FActorDragDropOp);
		ActorOperation->Init(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(FWeakDataLayerActorSelector()));
		ActorOperation->Construct();
		return ActorOperation;
	};

	auto GetActorDataLayerOperation = [&DraggedObjects]()
	{
		TSharedPtr<FDataLayerActorMoveOp> DataLayerActorOperation = MakeShareable(new FDataLayerActorMoveOp);
		DataLayerActorOperation->DataLayerActorMoveElements = DraggedObjects.GetData<FDataLayerActorMoveElement>(FDataLayerActorPairSelector());
		DataLayerActorOperation->Construct();
		return DataLayerActorOperation;
	};

	if (DraggedObjects.Has<FDataLayerTreeItem>() && !DraggedObjects.Has<FDataLayerActorTreeItem>())
	{
		return GetDataLayerOperation();
	}
	else if (!DraggedObjects.Has<FDataLayerTreeItem>() && DraggedObjects.Has<FDataLayerActorTreeItem>())
	{	
		if (MouseEvent.IsLeftAltDown())
		{
			return GetActorDataLayerOperation();
		}
		else
		{
			return GetActorOperation();
		}
	}
	else
	{
		TSharedPtr<FSceneOutlinerDragDropOp> OutlinerOp = MakeShareable(new FSceneOutlinerDragDropOp());

		if (DraggedObjects.Has<FDataLayerActorTreeItem>())
		{
			OutlinerOp->AddSubOp(GetActorOperation());
		}

		if (DraggedObjects.Has<FDataLayerTreeItem>())
		{
			OutlinerOp->AddSubOp(GetDataLayerOperation());
		}

		OutlinerOp->Construct();
		return OutlinerOp;
	}
}

static const FName DefaultContextBaseMenuName("DataLayerOutliner.DefaultContextMenuBase");
static const FName DefaultContextMenuName("DataLayerOutliner.DefaultContextMenu");

TArray<UDataLayerInstance*> FDataLayerMode::GetSelectedDataLayers(SSceneOutliner* InSceneOutliner) const
{
	FSceneOutlinerItemSelection ItemSelection(InSceneOutliner->GetSelection());
	TArray<FDataLayerTreeItem*> SelectedDataLayerItems;
	ItemSelection.Get<FDataLayerTreeItem>(SelectedDataLayerItems);
	TArray<UDataLayerInstance*> ValidSelectedDataLayers;
	Algo::TransformIf(SelectedDataLayerItems, ValidSelectedDataLayers, [](const auto Item) { return Item && Item->GetDataLayer(); }, [](const auto Item) { return Item->GetDataLayer(); });
	return MoveTemp(ValidSelectedDataLayers);
}

void FDataLayerMode::CreateDataLayerPicker(UToolMenu* InMenu, FOnDataLayerInstancePicked OnDataLayerInstancePicked, FOnShouldFilterDataLayerInstance OnShouldFilterDataLayerInstance = FOnShouldFilterDataLayerInstance(), bool bInShowRoot /*= false*/)
{
	FToolMenuSection& Section = InMenu->AddSection("DataLayers", LOCTEXT("DataLayers", "Data Layers"));
	if (bInShowRoot)
	{
		Section.AddMenuEntry("Root", LOCTEXT("Root", "<Root>"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([=]() { OnDataLayerInstancePicked.ExecuteIfBound(nullptr); })));
	}
	TSharedRef<SWidget> DataLayerPickerWidget = FDataLayerPickingMode::CreateDataLayerPickerWidget(OnDataLayerInstancePicked, OnShouldFilterDataLayerInstance);
	Section.AddEntry(FToolMenuEntry::InitWidget("DataLayerPickerWidget", DataLayerPickerWidget, FText::GetEmpty(), false));
}

UWorld* FDataLayerMode::GetOwningWorld() const
{
	UWorld* World = RepresentingWorld.Get();
	return World ? World->PersistentLevel->GetWorld() : nullptr;
}

AWorldDataLayers* FDataLayerMode::GetOwningWorldAWorldDataLayers() const
{
	UWorld* OwningWorld = GetOwningWorld();
	return OwningWorld ? OwningWorld->GetWorldDataLayers() : nullptr;
}

void FDataLayerMode::FindInContentBrowser()
{
	if (SceneOutliner)
	{
		TArray<UObject*> Objects;
		for (TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance : SelectedDataLayersSet)
		{
			if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance.Get()))
			{
				if (const UDataLayerAsset* Asset = DataLayerInstanceWithAsset->GetAsset())
				{
					Objects.Add(const_cast<UDataLayerAsset*>(Asset));
				}
			}
		}
		if (!Objects.IsEmpty())
		{
			GEditor->SyncBrowserToObjects(Objects);
		}
	}
}

bool FDataLayerMode::CanFindInContentBrowser() const
{
	for (const TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance : SelectedDataLayersSet)
	{
		if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance.Get()))
		{
			if (const UDataLayerAsset* Asset = DataLayerInstanceWithAsset->GetAsset())
			{
				return true;
			}
		}
	}
	return false;
}

void FDataLayerMode::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(DefaultContextBaseMenuName);

		Menu->AddDynamicSection("DataLayerDynamicSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
			if (!Context || !Context->SceneOutliner.IsValid())
			{
				return;
			}

			SSceneOutliner* SceneOutliner = Context->SceneOutliner.Pin().Get();
			FDataLayerMode* Mode = const_cast<FDataLayerMode*>(static_cast<const FDataLayerMode*>(SceneOutliner->GetMode()));
			check(Mode);
			TArray<UDataLayerInstance*> SelectedDataLayers = Mode->GetSelectedDataLayers(SceneOutliner);
			const bool bSelectedDataLayersContainsLocked = Algo::AnyOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsReadOnly(); });
			const bool bSelectedDataLayersContainsExternalPackage = Algo::AnyOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsPackageExternal(); });

			TArray<AActor*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
			const bool bSelectedActorsAreAllUserManaged = Algo::AllOf(SelectedActors, [](const AActor* Actor) { return Actor->IsUserManaged(); });
			static const FText SelectionContainsNonUserManagedActorsText = LOCTEXT("SelectionContainsUserManagedActors", "Selection Contains Non User Managed Actors");

			bool bHasActorEditorContextDataLayers = false;			
			TArray<const UDataLayerInstance*> AllDataLayers;
			if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(Mode->GetOwningWorld()))
			{
				DataLayerManager->ForEachDataLayerInstance([&AllDataLayers](UDataLayerInstance* DataLayerInstance)
				{
					AllDataLayers.Add(DataLayerInstance);
					return true;
				});

				bHasActorEditorContextDataLayers = !DataLayerManager->GetActorEditorContextDataLayers().IsEmpty();
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayers", LOCTEXT("DataLayers", "Data Layers"));
				
				auto GetBestCandidateWorldDataLayersFromSelection = [](SSceneOutliner* SceneOutliner, FDataLayerMode* Mode)
				{
					FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
					TArray<FWorldDataLayersTreeItem*> SelectedWorldDataLayersItems;
					Selection.Get<FWorldDataLayersTreeItem>(SelectedWorldDataLayersItems);
					TArray<AWorldDataLayers*> SelectedDataLayers;
					Algo::TransformIf(SelectedWorldDataLayersItems, SelectedDataLayers, [](const FWorldDataLayersTreeItem* Item) { return Item && Item->GetWorldDataLayers(); }, [](const FWorldDataLayersTreeItem* Item) { return Item->GetWorldDataLayers(); });
					AWorldDataLayers* BestCandidate = (SelectedDataLayers.Num() == 1) ? SelectedDataLayers[0] : nullptr;
					if (!BestCandidate)
					{
						TSet<AWorldDataLayers*> OuterWorldDataLayers;
						Algo::TransformIf(Mode->SelectedDataLayersSet, OuterWorldDataLayers, [](const TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance) { return DataLayerInstance.IsValid() && DataLayerInstance->GetDirectOuterWorldDataLayers(); }, [](const TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance) { return DataLayerInstance->GetDirectOuterWorldDataLayers(); });
						if (OuterWorldDataLayers.Num() == 1)
						{
							BestCandidate = *OuterWorldDataLayers.CreateConstIterator();
						}
					}
					if (!BestCandidate)
					{
						TSet<AWorldDataLayers*> OuterWorldDataLayers;
						Algo::TransformIf(Mode->SelectedDataLayerActors, OuterWorldDataLayers, [](const FSelectedDataLayerActor& DataLayerActor) { return DataLayerActor.Key.IsValid() && DataLayerActor.Key->GetDirectOuterWorldDataLayers(); }, [](const FSelectedDataLayerActor& DataLayerActor) { return DataLayerActor.Key->GetDirectOuterWorldDataLayers(); });
						if (OuterWorldDataLayers.Num() == 1)
						{
							BestCandidate = *OuterWorldDataLayers.CreateConstIterator();
						}
					}
					if (!BestCandidate)
					{
						BestCandidate = Mode->GetOwningWorldAWorldDataLayers();
					}
					return BestCandidate;
				};

				auto CreateNewDataLayerInternal = [&GetBestCandidateWorldDataLayersFromSelection, SceneOutliner, Mode](const UDataLayerAsset* InDataLayerAsset = nullptr, bool bInIsPrivate = false) -> UDataLayerInstance*
				{
					const FScopedTransaction Transaction(LOCTEXT("CreateNewDataLayer", "Create New Data Layer"));
					AWorldDataLayers* TargetWorldDataLayers = GetBestCandidateWorldDataLayersFromSelection(SceneOutliner, Mode);
					Mode->SelectedDataLayersSet.Empty();
					Mode->SelectedDataLayerActors.Empty();

					FDataLayerCreationParameters CreationParams;
					CreationParams.DataLayerAsset = const_cast<UDataLayerAsset*>(InDataLayerAsset);
					CreationParams.WorldDataLayers = TargetWorldDataLayers;
					CreationParams.bIsPrivate = bInIsPrivate;
					if (UDataLayerInstance* NewDataLayerInstance = UDataLayerEditorSubsystem::Get()->CreateDataLayerInstance(CreationParams))
					{
						Mode->SelectedDataLayersSet.Add(NewDataLayerInstance);
						// Select it and open a rename when it gets refreshed
						SceneOutliner->OnItemAdded(NewDataLayerInstance, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
						return NewDataLayerInstance;
					}
					return nullptr;
				};

				auto CreateNewDataLayer = [SceneOutliner, Mode, CreateNewDataLayerInternal](UDataLayerInstance* InParentDataLayer = nullptr, const UDataLayerAsset* InDataLayerAsset = nullptr, bool bInIsPrivate = false) -> UDataLayerInstance*
				{
					return CreateNewDataLayerInternal(InDataLayerAsset, bInIsPrivate);
				};

				auto CreateNewEmptyDataLayer = [SceneOutliner, Mode, CreateNewDataLayerInternal](bool bInIsPrivate = false) -> UDataLayerInstance*
				{
					return CreateNewDataLayerInternal(nullptr, bInIsPrivate);
				};

				const AWorldDataLayers* OwningWorldAWorldDataLayers = Mode->GetOwningWorldAWorldDataLayers();
				if (OwningWorldAWorldDataLayers)
				{
					const AWorldDataLayers* TargetWorldDataLayers = GetBestCandidateWorldDataLayersFromSelection(SceneOutliner, Mode);
					Section.AddSubMenu("CreateNewDataLayerWithAsset", LOCTEXT("CreateNewDataLayerWithAssetSubMenu", "Create New Data Layer With Asset"), LOCTEXT("CreateNewDataLayerWithAssetSubMenu_ToolTip", "Create New Data Layer With Asset"),
						FNewToolMenuDelegate::CreateLambda([Mode, CreateNewDataLayer, OwningWorldAWorldDataLayers, TargetWorldDataLayers](UToolMenu* InSubMenu)
						{
							const bool bAllowClear = false;
							const bool bAllowCopyPaste = false;
							const TArray<const UClass*> AllowedClasses = { UDataLayerAsset::StaticClass() };
							const TArray<const UClass*> NewAssetDisallowedClasses = { UExternalDataLayerAsset::StaticClass() };
							FToolMenuSection& Section = InSubMenu->AddSection("Data Layer Asset");
							TSharedRef<SWidget> MenuWidget = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
								FAssetData(),
								bAllowClear,
								bAllowCopyPaste,
								AllowedClasses,
								PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses, NewAssetDisallowedClasses),
								FOnShouldFilterAsset::CreateLambda([Mode, OwningWorldAWorldDataLayers, TargetWorldDataLayers](const FAssetData& InAssetData)
								{
									const UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(InAssetData.GetAsset());
									const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(DataLayerAsset);

									// Filter all External Data Layer Assets except if the target WorldDataLayers is the main one
									if (ExternalDataLayerAsset && (TargetWorldDataLayers != OwningWorldAWorldDataLayers))
									{
										return true;
									}

									// Skip asset filtering for External Data Layer asset if target WorldDataLayers is the main one
									if (!ExternalDataLayerAsset || (TargetWorldDataLayers != OwningWorldAWorldDataLayers))
									{
										if (!Mode->CanReferenceDataLayerAssets(TargetWorldDataLayers, { DataLayerAsset }))
										{
											return true;
										}
									}

									// Filter already used Data Layers Assets and External Data Layer Assets that can't be added (those already added or not part of a registered GFD action)
									const UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(Mode->GetOwningWorld());
									const bool bCanInjectExternalDataLayerAsset = ExternalDataLayerAsset && ExternalDataLayerManager && ExternalDataLayerManager->CanInjectExternalDataLayerAsset(ExternalDataLayerAsset);
									return !DataLayerAsset || (ExternalDataLayerAsset && !bCanInjectExternalDataLayerAsset) || OwningWorldAWorldDataLayers->GetDataLayerInstance(DataLayerAsset);
								}),
								FOnAssetSelected::CreateLambda([CreateNewDataLayer](const FAssetData& InAssetData)
								{
									if (UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(InAssetData.GetAsset()))
									{
										CreateNewDataLayer(nullptr, DataLayerAsset);
									}
								}),
								FSimpleDelegate::CreateLambda([]() { FSlateApplication::Get().DismissAllMenus(); }));
							Section.AddEntry(FToolMenuEntry::InitWidget("PickDataLayerAsset", MenuWidget, FText::GetEmpty(), false));
						}), false, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Import"));
				}
				else
				{
					Section.AddMenuEntry("CreateNewDataLayer", LOCTEXT("CreateNewDataLayer", "Create New Data Layer"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.AddCircle"),
						FUIAction(FExecuteAction::CreateLambda([CreateNewEmptyDataLayer]() { CreateNewEmptyDataLayer(); })));
				}

				Section.AddMenuEntry("CreateNewDataLayerPrivate", LOCTEXT("CreateNewDataLayerPrivate", "Create New Private Data Layer"), LOCTEXT("CreateNewDataLayerPrivateToolTip", "Creates an Editor Only Data Layer that cannot be used by other Worlds."), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.AddCircle"),
					FUIAction(FExecuteAction::CreateLambda([CreateNewEmptyDataLayer]() { CreateNewEmptyDataLayer(true); } )));

				bool bCanRename = false;
				if (SelectedDataLayers.Num() == 1)
				{
					FSceneOutlinerTreeItemPtr ItemToRename = SceneOutliner->GetSelection().SelectedItems[0].Pin();
					if (ItemToRename.IsValid() && Mode->CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
					{
						bCanRename = true;
					}
				}

				Section.AddMenuEntry("RenameSelectedDataLayer", LOCTEXT("RenameSelectedDataLayer", "Rename Selected Data Layer"), (bCanRename) ? FText() : LOCTEXT("RenameSelectedDataLayerTooltip", "Only Private data layers can be renamed, as Asset data layers take their name from the asset. Also a single data layer can be renamed at a time"), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Rename"),
					FUIAction(
						FExecuteAction::CreateLambda([Mode, SelectedDataLayers, SceneOutliner]() 
						{
							if (SelectedDataLayers.Num() == 1)
							{
								FSceneOutlinerTreeItemPtr ItemToRename = SceneOutliner->GetSelection().SelectedItems[0].Pin();
								if (ItemToRename.IsValid() && Mode->CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
								{
									SceneOutliner->SetPendingRenameItem(ItemToRename);
									SceneOutliner->ScrollItemIntoView(ItemToRename);
								}
							}
						}),
						FCanExecuteAction::CreateLambda([bCanRename]
						{
							return bCanRename;
						})
					), EUserInterfaceActionType::Button, NAME_None, LOCTEXT("F2", "F2"));

				const bool bAllSelectedDataLayersCanBeRemoved = Algo::AllOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->CanBeRemoved(); });
				Section.AddMenuEntry("DeleteSelectedDataLayers", LOCTEXT("DeleteSelectedDataLayers", "Delete Selected Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Delete"),
					FUIAction(
						FExecuteAction::CreateLambda([Mode, SelectedDataLayers]() 
						{
							check(!SelectedDataLayers.IsEmpty());
							Mode->DeleteDataLayers(SelectedDataLayers);
						}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers, bSelectedDataLayersContainsLocked, bAllSelectedDataLayersCanBeRemoved]
						{
							return !SelectedDataLayers.IsEmpty() && !bSelectedDataLayersContainsLocked && bAllSelectedDataLayersCanBeRemoved;
						})
					), EUserInterfaceActionType::Button, NAME_None, LOCTEXT("Delete", "Delete"));

				const bool bSelectedDataLayersSupportParenting = Algo::AllOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return !DataLayerInstance->IsReadOnly() && DataLayerInstance->CanHaveParentDataLayerInstance(); });
				if (!SelectedDataLayers.IsEmpty() && !bSelectedDataLayersContainsLocked && bSelectedDataLayersSupportParenting)
				{
					Section.AddSubMenu("MoveSelectedDataLayersTo", LOCTEXT("MoveSelectedDataLayersTo", "Move Selected Data Layer(s) To"), FText(),
						FNewToolMenuDelegate::CreateLambda([Mode, SelectedDataLayers](UToolMenu* InSubMenu)
						{
							const bool bShowRoot = Algo::AllOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->CanBeChildOf(nullptr); });
							CreateDataLayerPicker(InSubMenu,
								FOnDataLayerInstancePicked::CreateLambda([Mode, SelectedDataLayers](UDataLayerInstance* TargetDataLayerInstance)
								{
									TArray<UDataLayerInstance*> DataLayerInstances;
									for (UDataLayerInstance* DataLayerInstance : SelectedDataLayers)
									{
										if (DataLayerInstance->CanBeChildOf(TargetDataLayerInstance))
										{
											DataLayerInstances.Add(DataLayerInstance);
										}
									}
									Mode->SetParentDataLayer(DataLayerInstances, TargetDataLayerInstance);
								}),
								FOnShouldFilterDataLayerInstance::CreateLambda([Mode, SelectedDataLayers](const UDataLayerInstance* InCandidateDataLayerInstance)
								{
									for (UDataLayerInstance* DataLayerInstance : SelectedDataLayers)
									{
										if (!DataLayerInstance->CanBeChildOf(InCandidateDataLayerInstance))
										{
											// True means should filter
											return true;
										}
									}
									return false;
								}),
								bShowRoot);
						}), false, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Reimport"));
				}

				Section.AddMenuEntry("Copy Selected Data Layer Instances(s) File Path", LOCTEXT("CopySelectedDataLayerInstancessFilePath", "Copy Selected Data Layer Instance(s) File Path"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]()
						{
							check(!SelectedDataLayers.IsEmpty());
							{
								TArray<const UObject*> Objects;
								Algo::Transform(SelectedDataLayers, Objects, [](UDataLayerInstance* DataLayerInstance) { return DataLayerInstance; });
								FExternalPackageHelper::CopyObjectsExternalPackageFilePathToClipboard(Objects);
							}
						}),
						FCanExecuteAction::CreateLambda([bSelectedDataLayersContainsExternalPackage] { return bSelectedDataLayersContainsExternalPackage; })
					));

				Section.AddSeparator("SectionsSeparator");


				const bool bAllSelectedDataLayersCanAddActors = Algo::AllOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->CanUserAddActors(); });
				FText AddActorToolTip = bSelectedActorsAreAllUserManaged ? FText::GetEmpty() : SelectionContainsNonUserManagedActorsText;
				Section.AddMenuEntry("AddSelectedActorsToSelectedDataLayers", LOCTEXT("AddSelectedActorsToSelectedDataLayers", "Add Selected Actor(s) to Selected Data Layer(s)"), AddActorToolTip, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "DataLayerBrowser.AddSelection"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]()
						{
							check(!SelectedDataLayers.IsEmpty());
							{
								const FScopedTransaction Transaction(LOCTEXT("AddSelectedActorsToSelectedDataLayers", "Add Selected Actor(s) to Selected Data Layer(s)"));
								UDataLayerEditorSubsystem::Get()->AddSelectedActorsToDataLayers(SelectedDataLayers);
							}
						}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers, bAllSelectedDataLayersCanAddActors, bSelectedActorsAreAllUserManaged]
						{
							return !SelectedDataLayers.IsEmpty() && GEditor->GetSelectedActorCount() > 0 && bAllSelectedDataLayersCanAddActors && bSelectedActorsAreAllUserManaged;
						})
					));

				if (!Mode->SelectedDataLayerActors.IsEmpty())
				{
					Section.AddSubMenu("AddSelectedActorsTo", LOCTEXT("AddSelectedActorsTo", "Add Selected Actor(s) To"), FText(),
						FNewToolMenuDelegate::CreateLambda([Mode](UToolMenu* InSubMenu)
						{
							CreateDataLayerPicker(InSubMenu, 
								FOnDataLayerInstancePicked::CreateLambda([Mode](UDataLayerInstance* TargetDataLayer)
								{
									check(TargetDataLayer);
									TArray<AActor*> Actors;
									Algo::TransformIf(Mode->SelectedDataLayerActors, Actors, [](const FSelectedDataLayerActor& InActor) { return InActor.Value.Get(); }, [](const FSelectedDataLayerActor& InActor) { return const_cast<AActor*>(InActor.Value.Get()); });
									if (!Actors.IsEmpty())
									{
										const FScopedTransaction Transaction(LOCTEXT("AddSelectedActorsToDataLayer", "Add Selected Actors to Selected Data Layer"));
										UDataLayerEditorSubsystem::Get()->AddActorsToDataLayers(Actors, { TargetDataLayer });
									}
								}),
								FOnShouldFilterDataLayerInstance::CreateLambda([Mode](const UDataLayerInstance* InCandidateDataLayerInstance)
								{
									for (auto& [DataLayerInstance, SelectedActor] : Mode->SelectedDataLayerActors)
									{
										if (!InCandidateDataLayerInstance->CanAddActor(const_cast<AActor*>(SelectedActor.Get())))
										{
											// True means should filter
											return true;
										}
									}
									return false;
								})
							);
						}), false, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "DataLayerBrowser.AddSelection"));
				}

				const bool bAllSelectedDataLayersCanRemoveActors = Algo::AllOf(SelectedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->CanUserRemoveActors(); });
				FText RemoveActorToolTip = bSelectedActorsAreAllUserManaged ? FText::GetEmpty() : SelectionContainsNonUserManagedActorsText;
				Section.AddMenuEntry("RemoveSelectedActorsFromSelectedDataLayers", LOCTEXT("RemoveSelectedActorsFromSelectedDataLayersMenu", "Remove Selected Actor(s) from Selected Data Layer(s)"), RemoveActorToolTip, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "DataLayerBrowser.RemoveSelection"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]() 
						{
							check(!SelectedDataLayers.IsEmpty());
							{
								const FScopedTransaction Transaction(LOCTEXT("RemoveSelectedActorsFromSelectedDataLayers_DataLayerMode", "Remove Selected Actors from Selected Data Layers"));
								UDataLayerEditorSubsystem::Get()->RemoveSelectedActorsFromDataLayers(SelectedDataLayers);
							}
						}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers, bAllSelectedDataLayersCanRemoveActors, bSelectedActorsAreAllUserManaged]
						{ 
							return !SelectedDataLayers.IsEmpty() && GEditor->GetSelectedActorCount() > 0 && bAllSelectedDataLayersCanRemoveActors && bSelectedActorsAreAllUserManaged;
						})
					));

				Section.AddSeparator("SectionsSeparator");
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayerSelection", LOCTEXT("DataLayerSelection", "Selection"));

				Section.AddMenuEntry("SelectActorsInDataLayers", LOCTEXT("SelectActorsInDataLayers", "Select Actor(s) in Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.SelectAll"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]() {
						check(!SelectedDataLayers.IsEmpty());
						{
							const FScopedTransaction Transaction(LOCTEXT("SelectActorsInDataLayers", "Select Actor(s) in Data Layer(s)"));
							GEditor->SelectNone(/*bNoteSelectionChange*/false, /*bDeselectBSPSurfs*/true);
							UDataLayerEditorSubsystem::Get()->SelectActorsInDataLayers(SelectedDataLayers, /*bSelect*/true, /*bNotify*/true, /*bSelectEvenIfHidden*/true);
						}}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));

				Section.AddMenuEntry("AddActorsToSelection", LOCTEXT("AddActorsToSelection", "Add Actors in Data Layer(s) to Selection"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Merge"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]() {
						check(!SelectedDataLayers.IsEmpty());
						{
							const FScopedTransaction Transaction(LOCTEXT("AppendActorsToSelection", "Append Actors in Data Layer to Selection"));
							UDataLayerEditorSubsystem::Get()->SelectActorsInDataLayers(SelectedDataLayers, /*bSelect*/true, /*bNotify*/true, /*bSelectEvenIfHidden*/true);
						}}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));

				Section.AddMenuEntry("DeselectActors", LOCTEXT("DeselectActors", "Deselect Actor(s) in Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.XCircle"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]() {
						check(!SelectedDataLayers.IsEmpty());
						{
							const FScopedTransaction Transaction(LOCTEXT("DeselectActors", "Deselect Actor(s) in Data Layer(s)"));
							UDataLayerEditorSubsystem::Get()->SelectActorsInDataLayers(SelectedDataLayers, /*bSelect*/false, /*bNotifySelectActors*/true);
						}}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayerLoading", LOCTEXT("DataLayerLoading", "Loading"));

				Section.AddMenuEntry("ForceLoadActorsInDataLayers", LOCTEXT("ForceLoadActorsInDataLayers", "Force Load Actor(s) in Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Pinned"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]() 
						{
							check(!SelectedDataLayers.IsEmpty());
							const FScopedTransaction Transaction(LOCTEXT("ForceLoadActorsInDataLayers", "Force Load Actor(s) in Data Layer(s)"));
							UDataLayerEditorSubsystem::Get()->SetActorsPinStateInDataLayers(SelectedDataLayers, true);
						}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));

				Section.AddMenuEntry("ReleaseForceLoadActorsInDataLayers", LOCTEXT("ReleaseForceLoadActorsInDataLayers", "Release Force Load Actor(s) in Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Unpinned"),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedDataLayers]()
							{
								check(!SelectedDataLayers.IsEmpty());
								const FScopedTransaction Transaction(LOCTEXT("ReleaseForceLoadActorsInDataLayers", "Release Force Load Actor(s) in Data Layer(s)"));
								UDataLayerEditorSubsystem::Get()->SetActorsPinStateInDataLayers(SelectedDataLayers, false);
							}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayerExpansion", LOCTEXT("DataLayerExpansion", "Expansion"));

				Section.AddMenuEntry("CollapseAllDataLayers", LOCTEXT("CollapseAllDataLayers", "Collapse All Data Layers"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.ChevronUp"),
					FUIAction(
						FExecuteAction::CreateLambda([SceneOutliner,SelectedDataLayers]() {
						check(!SelectedDataLayers.IsEmpty());
						{
							GEditor->SelectNone(/*bNoteSelectionChange*/false, /*bDeselectBSPSurfs*/true);
							SceneOutliner->CollapseAll();
						}}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));

				Section.AddMenuEntry("ExpandAllDataLayers", LOCTEXT("ExpandAllDataLayers", "Expand All Data Layers"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.ChevronDown"),
					FUIAction(
						FExecuteAction::CreateLambda([SceneOutliner,SelectedDataLayers]() {
						check(!SelectedDataLayers.IsEmpty());
						{
							GEditor->SelectNone(/*bNoteSelectionChange*/false, /*bDeselectBSPSurfs*/true);
							SceneOutliner->ExpandAll();
						}}),
						FCanExecuteAction::CreateLambda([SelectedDataLayers] { return !SelectedDataLayers.IsEmpty(); })
					));
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayerActorEditorContext", LOCTEXT("DataLayerActorEditorContext", "Actor Editor Context"));

				Section.AddMenuEntry("MakeCurrentDataLayers", LOCTEXT("MakeCurrentDataLayers", "Make Current Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Check"),
					FUIAction(
						FExecuteAction::CreateLambda([Mode, SceneOutliner, SelectedDataLayers]() 
						{
							const FScopedTransaction Transaction(LOCTEXT("MakeCurrentDataLayers", "Make Current Data Layer(s)"));
							for (TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance : Mode->SelectedDataLayersSet)
							{
								if (DataLayerInstance.IsValid())
								{
									UDataLayerEditorSubsystem::Get()->AddToActorEditorContext(const_cast<UDataLayerInstance*>(DataLayerInstance.Get()));
								}
							}
						}),
						FCanExecuteAction::CreateLambda([Mode] 
						{ 
							return Algo::AnyOf(Mode->SelectedDataLayersSet, [](const TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance)
							{ 
								return DataLayerInstance.IsValid() && DataLayerInstance->CanBeInActorEditorContext() && !DataLayerInstance->IsInActorEditorContext();
							});
						})
					));

				Section.AddMenuEntry("RemoveCurrentDataLayers", LOCTEXT("RemoveCurrentDataLayers", "Remove Current Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.X"),
					FUIAction(
						FExecuteAction::CreateLambda([Mode, SceneOutliner, SelectedDataLayers]()
						{
							const FScopedTransaction Transaction(LOCTEXT("RemoveCurrentDataLayers", "Remove Current Data Layer(s)"));
							for (TWeakObjectPtr<const UDataLayerInstance>& DataLayerInstance : Mode->SelectedDataLayersSet)
							{
								if (DataLayerInstance.IsValid())
								{
									UDataLayerEditorSubsystem::Get()->RemoveFromActorEditorContext(const_cast<UDataLayerInstance*>(DataLayerInstance.Get()));
								}
							}
						}),
						FCanExecuteAction::CreateLambda([Mode] 
						{ 
							return Algo::AnyOf(Mode->SelectedDataLayersSet, [](const TWeakObjectPtr<const UDataLayerInstance>& DatalLayerInstance) 
							{ 
								return DatalLayerInstance.IsValid() && DatalLayerInstance->CanBeInActorEditorContext() && DatalLayerInstance->IsInActorEditorContext();
							});
						})
					));

				Section.AddMenuEntry("ClearCurrentDataLayers", LOCTEXT("ClearCurrentDataLayers", "Clear Current Data Layer(s)"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Delete"),
					FUIAction(
						FExecuteAction::CreateLambda([AllDataLayers]() 
						{
							check(!AllDataLayers.IsEmpty());
							{
								const FScopedTransaction Transaction(LOCTEXT("ClearCurrentDataLayers", "Clear Current Data Layer(s)"));
								for (const UDataLayerInstance* DataLayer : AllDataLayers)
								{
									UDataLayerEditorSubsystem::Get()->RemoveFromActorEditorContext(const_cast<UDataLayerInstance*>(DataLayer));
								}
							}
						}),
						FCanExecuteAction::CreateLambda([bHasActorEditorContextDataLayers] { return bHasActorEditorContextDataLayers; })
					));
			}

			{
				FToolMenuSection& Section = InMenu->AddSection("DataLayerVisibility", LOCTEXT("DataLayerVisibility", "Visibility"));

				Section.AddMenuEntry("MakeAllDataLayersVisible", LOCTEXT("MakeAllDataLayersVisible", "Make All Data Layers Visible"), FText(), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "Icons.Visible"),
					FUIAction(
						FExecuteAction::CreateLambda([AllDataLayers]()
						{
							check(!AllDataLayers.IsEmpty());
							{
								const FScopedTransaction Transaction(LOCTEXT("MakeAllDataLayersVisible", "Make All Data Layers Visible"));
								UDataLayerEditorSubsystem::Get()->MakeAllDataLayersVisible();
							}
						}),
						FCanExecuteAction::CreateLambda([AllDataLayers] { return !AllDataLayers.IsEmpty(); })
					));
			}

			FToolMenuSection& Section = InMenu->AddSection("AssetOptionsSection", LOCTEXT("AssetOptionsText", "Asset Options"));
			Section.AddMenuEntryWithCommandList(FGlobalEditorCommonCommands::Get().FindInContentBrowser, Mode->Commands);
			const UExternalDataLayerInstance* ExternalDataLayerInstance = SelectedDataLayers.Num() == 1 ? Cast<UExternalDataLayerInstance>(SelectedDataLayers[0]) : nullptr;
			if (const UExternalDataLayerAsset* ExternalDataLayerAsset = ExternalDataLayerInstance ? ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr)
			{
				Section.AddMenuEntry("BrowseToPluginAsset", LOCTEXT("BrowseToPluginAsset", "Browse to Plugin Asset"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
					FUIAction(
						FExecuteAction::CreateLambda([ExternalDataLayerAsset]()
						{
							UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
							TArray<UObject*> Objects;;
							for (UObject* Client : ExternalDataLayerEngineSubsystem.GetClientsForExternalDataLayerAsset(ExternalDataLayerAsset))
							{
								if (UDataAsset* DataAsset = Client->GetTypedOuter<UDataAsset>())
								{
									Objects.Add(DataAsset);
								}
							}
							if (!Objects.IsEmpty())
							{
								GEditor->SyncBrowserToObjects(Objects);
							}
						})
					));
			}
		}));
	}

	if (!ToolMenus->IsMenuRegistered(DefaultContextMenuName))
	{
		ToolMenus->RegisterMenu(DefaultContextMenuName, DefaultContextBaseMenuName);
	}
}

TSharedPtr<SWidget> FDataLayerMode::CreateContextMenu()
{
	RegisterContextMenu();

	FSceneOutlinerItemSelection ItemSelection(SceneOutliner->GetSelection());
	CacheSelectedItems(ItemSelection);

	USceneOutlinerMenuContext* ContextObject = NewObject<USceneOutlinerMenuContext>();
	ContextObject->SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
	ContextObject->bShowParentTree = SceneOutliner->GetSharedData().bShowParentTree;
	ContextObject->NumSelectedItems = ItemSelection.Num();
	FToolMenuContext Context(ContextObject);

	FName MenuName = DefaultContextMenuName;
	SceneOutliner->GetSharedData().ModifyContextMenu.ExecuteIfBound(MenuName, Context);

	// Add Extenders
	FDataLayerEditorModule& DataLayerEditorModule = FModuleManager::LoadModuleChecked<FDataLayerEditorModule>("DataLayerEditor");
	TArray<FDataLayerEditorModule::FDataLayersMenuExtender>& MenuExtenderDelegates = DataLayerEditorModule.GetAllDataLayersMenuExtenders();
	TArray<TSharedPtr<FExtender>> Extenders;
	if (!MenuExtenderDelegates.IsEmpty())
	{
		for (const FDataLayerEditorModule::FDataLayersMenuExtender& MenuExtender : MenuExtenderDelegates)
		{
			if (MenuExtender.IsBound())
			{
				Extenders.Add(MenuExtender.Execute(Commands.ToSharedRef(), SelectedDataLayersSet, SelectedDataLayerActors));
			}
		}
		if (!Extenders.IsEmpty())
		{
			Context.AddExtender(FExtender::Combine(Extenders));
		}
	}

	// Build up the menu for a selection
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}

void FDataLayerMode::InitializeViewMenuExtender(TSharedPtr<FExtender> Extender)
{
	Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleHideLevelInstanceContent", "Hide Level Instance Content"),
			LOCTEXT("ToggleHideLevelInstanceContentToolTip", "When enabled, hides all level instance content."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
				bHideLevelInstanceContent = !bHideLevelInstanceContent;
				Settings->bHideLevelInstanceContent = bHideLevelInstanceContent;
				Settings->PostEditChange();

				if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
				{
					DataLayerHierarchy->SetShowLevelInstanceContent(!bHideLevelInstanceContent);
				}

				SceneOutliner->FullRefresh();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return bHideLevelInstanceContent; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}));

	Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleHighlightSelectedDataLayers", "Highlight Selected"),
			LOCTEXT("ToggleHighlightSelectedDataLayersToolTip", "When enabled, highlights Data Layers containing actors that are currently selected."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				UWorldPartitionEditorPerProjectUserSettings* Settings = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>();
				bHighlightSelectedDataLayers = !bHighlightSelectedDataLayers;
				Settings->bHighlightSelectedDataLayers = bHighlightSelectedDataLayers;
				Settings->PostEditChange();

				if (auto DataLayerHierarchy = StaticCast<FDataLayerHierarchy*>(Hierarchy.Get()))
				{
					DataLayerHierarchy->SetHighlightSelectedDataLayers(bHighlightSelectedDataLayers);
				}

				SceneOutliner->FullRefresh();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return bHighlightSelectedDataLayers; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.BeginSection("Advanced", LOCTEXT("ShowAdvancedHeading", "Advanced"));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAllowRuntimeDataLayerEditing", "Allow Runtime Data Layer Editing"),
			LOCTEXT("ToggleAllowRuntimeDataLayerEditingToolTip", "When enabled, allows editing of Runtime Data Layers."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				if (AWorldDataLayers* WorldDataLayers = RepresentingWorld.IsValid() ? RepresentingWorld->GetWorldDataLayers() : nullptr)
				{
					const FScopedTransaction Transaction(LOCTEXT("ToggleAllowRuntimeDataLayerEditingTransaction", "Toggle Allow Runtime Data Layer Editing"));
					WorldDataLayers->SetAllowRuntimeDataLayerEditing(!WorldDataLayers->GetAllowRuntimeDataLayerEditing());
				}
				SceneOutliner->FullRefresh();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]()
			{
				const AWorldDataLayers* WorldDataLayers = RepresentingWorld.IsValid() ? RepresentingWorld->GetWorldDataLayers() : nullptr;
				return WorldDataLayers ? WorldDataLayers->GetAllowRuntimeDataLayerEditing() : true;
			})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		TArray<UDataLayerInstance*> AllDataLayers;
		if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetOwningWorld()))
		{
			DataLayerManager->ForEachDataLayerInstance([&AllDataLayers](UDataLayerInstance* DataLayer)
			{
				AllDataLayers.Add(DataLayer);
				return true;
			});
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetDataLayerUserSettings", "Reset User Settings"),
			LOCTEXT("ResetDataLayerUserSettingsToolTip", "Resets Data Layers User Settings to their initial values."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				const FScopedTransaction Transaction(LOCTEXT("ResetDataLayerUserSettings", "Reset User Settings"));
				DataLayerEditorSubsystem->ResetUserSettings();
			}))
		);

		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("World", LOCTEXT("ShowWorldHeading", "World"));

		MenuBuilder.AddSubMenu(
			LOCTEXT("ChooseWorldSubMenu", "Choose World"),
			LOCTEXT("ChooseWorldSubMenuToolTip", "Choose the world to display in the outliner."),
			FNewMenuDelegate::CreateRaw(this, &FDataLayerMode::BuildWorldPickerMenu)
		);
		MenuBuilder.EndSection();
	}));
}

void FDataLayerMode::BuildWorldPickerMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Worlds", LOCTEXT("WorldsHeading", "Worlds"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoWorld", "Auto"),
			LOCTEXT("AutoWorldToolTip", "Automatically pick the world to display based on context."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FDataLayerMode::OnSelectWorld, TWeakObjectPtr<UWorld>()),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FDataLayerMode::IsWorldChecked, TWeakObjectPtr<UWorld>())
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && (World->WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Editor))
			{
				MenuBuilder.AddMenuEntry(
					SceneOutliner::GetWorldDescription(World),
					LOCTEXT("ChooseWorldToolTip", "Display actors for this world."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FDataLayerMode::OnSelectWorld, MakeWeakObjectPtr(World)),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(this, &FDataLayerMode::IsWorldChecked, MakeWeakObjectPtr(World))
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	MenuBuilder.EndSection();
}

void FDataLayerMode::OnSelectWorld(TWeakObjectPtr<UWorld> World)
{
	UserChosenWorld = World;
	SceneOutliner->FullRefresh();
}

bool FDataLayerMode::IsWorldChecked(TWeakObjectPtr<UWorld> World) const
{
	return (UserChosenWorld == World) || (World.IsExplicitlyNull() && !UserChosenWorld.IsValid());
}

TUniquePtr<ISceneOutlinerHierarchy> FDataLayerMode::CreateHierarchy()
{
	TUniquePtr<FDataLayerHierarchy> DataLayerHierarchy = FDataLayerHierarchy::Create(this, RepresentingWorld);
	DataLayerHierarchy->SetShowEditorDataLayers(!bHideEditorDataLayers);
	DataLayerHierarchy->SetShowRuntimeDataLayers(!bHideRuntimeDataLayers);
	DataLayerHierarchy->SetShowDataLayerActors(!bHideDataLayerActors);
	DataLayerHierarchy->SetShowUnloadedActors(!bHideUnloadedActors);
	DataLayerHierarchy->SetShowOnlySelectedActors(bShowOnlySelectedActors);
	DataLayerHierarchy->SetHighlightSelectedDataLayers(bHighlightSelectedDataLayers);
	DataLayerHierarchy->SetShowLevelInstanceContent(!bHideLevelInstanceContent);
	return DataLayerHierarchy;
}


void FDataLayerMode::CacheSelectedItems(const FSceneOutlinerItemSelection& Selection)
{
	SelectedDataLayersSet.Empty();
	SelectedDataLayerActors.Empty();
	Selection.ForEachItem<FDataLayerTreeItem>([this](const FDataLayerTreeItem& Item) { SelectedDataLayersSet.Add(Item.GetDataLayer()); });
	Selection.ForEachItem<FDataLayerActorTreeItem>([this](const FDataLayerActorTreeItem& Item) { SelectedDataLayerActors.Add(FSelectedDataLayerActor(Item.GetDataLayer(), Item.GetActor())); });
}

void FDataLayerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	CacheSelectedItems(Selection);

	if (DataLayerBrowser)
	{
		DataLayerBrowser->OnSelectionChanged(SelectedDataLayersSet);
	}

	if (OnItemPicked.IsBound())
	{
		auto SelectedItems = SceneOutliner->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			auto FirstItem = SelectedItems[0];
			if (FirstItem->CanInteract())
			{
				OnItemPicked.ExecuteIfBound(FirstItem.ToSharedRef());
			}
		}
	}
}

void FDataLayerMode::Rebuild()
{
	FilteredDataLayerCount = 0;
	ApplicableDataLayers.Empty();
	ChooseRepresentingWorld();
	Hierarchy = CreateHierarchy();

	// Hide delete actor column when it's not necessary
	const bool bShowDeleteButtonColumn = !bHideDataLayerActors && RepresentingWorld.IsValid() && !RepresentingWorld->IsPlayInEditor();
	SceneOutliner->SetColumnVisibility(FDataLayerOutlinerDeleteButtonColumn::GetID(), bShowDeleteButtonColumn);

	if (DataLayerBrowser)
	{
		DataLayerBrowser->OnSelectionChanged(SelectedDataLayersSet);
	}
}

void FDataLayerMode::ChooseRepresentingWorld()
{
	// Select a world to represent

	RepresentingWorld = nullptr;

	// If a specified world was provided, represent it
	if (SpecifiedWorldToDisplay.IsValid())
	{
		RepresentingWorld = SpecifiedWorldToDisplay.Get();
	}

	// check if the user-chosen world is valid and in the editor contexts

	if (!RepresentingWorld.IsValid() && UserChosenWorld.IsValid())
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (UserChosenWorld.Get() == Context.World())
			{
				RepresentingWorld = UserChosenWorld.Get();
				break;
			}
		}
	}

	// If the user did not manually select a world, try to pick the most suitable world context
	if (!RepresentingWorld.IsValid())
	{
		// ideally we want a PIE world that is standalone or the first client
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && Context.WorldType == EWorldType::PIE)
			{
				if (World->GetNetMode() == NM_Standalone)
				{
					RepresentingWorld = World;
					break;
				}
				else if (World->GetNetMode() == NM_Client && Context.PIEInstance == 2)	// Slightly dangerous: assumes server is always PIEInstance = 1;
				{
					RepresentingWorld = World;
					break;
				}
			}
		}
	}

	if (RepresentingWorld == nullptr)
	{
		// still not world so fallback to old logic where we just prefer PIE over Editor
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				RepresentingWorld = Context.World();
				break;
			}
			else if (Context.WorldType == EWorldType::Editor)
			{
				RepresentingWorld = Context.World();
			}
		}
	}
}

bool FDataLayerMode::ShouldExpandDataLayer(const UDataLayerInstance* DataLayer) const
{
	if (bHighlightSelectedDataLayers || bShowOnlySelectedActors)
	{
		if (DataLayer)
		{
			if ((bShowOnlySelectedActors && DataLayerEditorSubsystem->DoesDataLayerContainSelectedActors(DataLayer)) ||
				(ContainsSelectedChildDataLayer(DataLayer) && !DataLayer->GetChildren().IsEmpty()))
			{
				return true;
			}
		}
	}
	return false;
}

bool FDataLayerMode::ContainsSelectedChildDataLayer(const UDataLayerInstance* DataLayer) const
{
	if (DataLayer)
	{
		bool bFoundSelected = false;
		DataLayer->ForEachChild([this, &bFoundSelected](const UDataLayerInstance* Child)
		{
			if (DataLayerEditorSubsystem->DoesDataLayerContainSelectedActors(Child) || ContainsSelectedChildDataLayer(Child))
			{
				bFoundSelected = true;
				return false;
			}
			return true;
		});
		return bFoundSelected;
	}
	return false;
};

TSharedRef<FSceneOutlinerFilter> FDataLayerMode::CreateShowOnlySelectedActorsFilter()
{
	auto IsActorSelected = [](const AActor* InActor, const UDataLayerInstance* InDataLayer)
	{
		return InActor && InActor->IsSelected();
	};
	return MakeShareable(new FDataLayerActorFilter(FDataLayerActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected), FSceneOutlinerFilter::EDefaultBehaviour::Pass, FDataLayerActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected)));
}

void FDataLayerMode::SynchronizeSelection()
{
	if (!bShowOnlySelectedActors && !bHighlightSelectedDataLayers)
	{
		return;
	}

	TArray<AActor*> Actors;
	TSet<const UDataLayerInstance*> ActorDataLayersIncludingParents;
	GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(Actors);
	for (const AActor* Actor : Actors)
	{
		TArray<const UDataLayerInstance*> ActorDataLayers = Actor->GetDataLayerInstances();
		for (const UDataLayerInstance* DataLayer : ActorDataLayers)
		{
			const UDataLayerInstance* CurrentDataLayer = DataLayer;
			while (CurrentDataLayer)
			{
				bool bIsAlreadyInSet = false;
				ActorDataLayersIncludingParents.Add(CurrentDataLayer, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(CurrentDataLayer, false);
					if (TreeItem && ShouldExpandDataLayer(CurrentDataLayer))
					{
						SceneOutliner->SetItemExpansion(TreeItem, true);
					}
				}
				CurrentDataLayer = CurrentDataLayer->GetParent();
			}
		}
	}
}

void FDataLayerMode::OnPostLoadMapWithWorld(UWorld* World)
{
	SceneOutliner->FullRefresh();
}

void FDataLayerMode::OnLevelSelectionChanged(UObject* Obj)
{
	if (!bShowOnlySelectedActors && !bHighlightSelectedDataLayers)
	{
		return;
	}

	RefreshSelection();
}

void FDataLayerMode::RefreshSelection()
{
	SceneOutliner->FullRefresh();
	SceneOutliner->RefreshSelection();
}

//
// FDataLayerPickingMode : Lightweight version of FDataLayerMode used to show the DataLayer hierarchy and choose one.
// 

FDataLayerPickingMode::FDataLayerPickingMode(const FDataLayerModeParams& Params) 
: FDataLayerMode(Params)
{
	bHideDataLayerActors = true;
	Rebuild();
	SceneOutliner->ExpandAll();
}

TSharedRef<SWidget> FDataLayerPickingMode::CreateDataLayerPickerWidget(FOnDataLayerInstancePicked OnDataLayerInstancePicked, FOnShouldFilterDataLayerInstance OnShouldFilterDataLayerInstance)
{
	// Create mini DataLayers outliner to pick a DataLayer
	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = false;
	InitOptions.bShowParentTree = true;
	InitOptions.bShowCreateNewFolder = false;
	InitOptions.bFocusSearchBoxWhenOpened = true;
	if (OnShouldFilterDataLayerInstance.IsBound())
	{
		InitOptions.Filters->AddFilterPredicate<FDataLayerTreeItem>(FDataLayerTreeItem::FFilterPredicate::CreateLambda([OnShouldFilterDataLayerInstance](const UDataLayerInstance* DataLayerInstance) { return !OnShouldFilterDataLayerInstance.Execute(DataLayerInstance); }), FSceneOutlinerFilter::EDefaultBehaviour::Pass);
	}
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 2));
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([OnDataLayerInstancePicked](SSceneOutliner* Outliner)
	{ 
		return new FDataLayerPickingMode(FDataLayerModeParams(Outliner, nullptr, nullptr, FOnSceneOutlinerItemPicked::CreateLambda([OnDataLayerInstancePicked](const FSceneOutlinerTreeItemRef& NewParent)
		{
			FDataLayerTreeItem* DataLayerItem = NewParent->CastTo<FDataLayerTreeItem>();
			if (UDataLayerInstance* DataLayer = DataLayerItem ? DataLayerItem->GetDataLayer() : nullptr)
			{
				OnDataLayerInstancePicked.ExecuteIfBound(DataLayer);
			}
			FSlateApplication::Get().DismissAllMenus();
		})));
	});

	TSharedRef<SDataLayerOutliner> Outliner = SNew(SDataLayerOutliner, InitOptions).IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
	TSharedRef<SWidget> DataLayerPickerWidget =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.MaxHeight(400.0f)
		[
			Outliner
		];

	Outliner->ExpandAll();

	return DataLayerPickerWidget;
}

void FDataLayerPickingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	if (OnItemPicked.IsBound())
	{
		auto SelectedItems = SceneOutliner->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			auto FirstItem = SelectedItems[0];
			if (FirstItem->CanInteract())
			{
				if (const FDataLayerTreeItem* DataLayerItem = FirstItem->CastTo<FDataLayerTreeItem>())
				{
					UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer();
					if (DataLayerInstance && !DataLayerInstance->IsReadOnly())
					{
						OnItemPicked.ExecuteIfBound(FirstItem.ToSharedRef());
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE