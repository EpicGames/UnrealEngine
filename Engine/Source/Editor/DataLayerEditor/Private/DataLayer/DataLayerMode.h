// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "DataLayer/DataLayerDragDropOp.h"
#include "Delegates/Delegate.h"
#include "Folder.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "ISceneOutlinerMode.h"
#include "ISceneOutlinerTreeItem.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "SSceneOutliner.h"
#include "SceneOutlinerDragDrop.h"
#include "SceneOutlinerFwd.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class AWorldDataLayers;
class FDragDropEvent;
class FDragDropOperation;
class FMenuBuilder;
class FSceneOutlinerFilter;
class FUICommandList;
class ISceneOutlinerHierarchy;
class SDataLayerBrowser;
class SWidget;
class UDataLayerAsset;
class UDataLayerEditorSubsystem;
class UDataLayerInstance;
class UObject;
class UToolMenu;
class UWorld;
struct FKeyEvent;
struct FPointerEvent;

struct FDataLayerModeParams
{
	FDataLayerModeParams() {}

	FDataLayerModeParams(SSceneOutliner* InSceneOutliner, SDataLayerBrowser* InDataLayerBrowser, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay = nullptr, FOnSceneOutlinerItemPicked InOnItemPicked = FOnSceneOutlinerItemPicked());

	TWeakObjectPtr<UWorld> SpecifiedWorldToDisplay = nullptr;
	SDataLayerBrowser* DataLayerBrowser;
	SSceneOutliner* SceneOutliner;
	FOnSceneOutlinerItemPicked OnItemPicked;
};

/** Called when data layer instance is picked in the Data Layer Picker */
DECLARE_DELEGATE_OneParam(FOnDataLayerInstancePicked, UDataLayerInstance*);

/** Called to check if a data layer instance should be filtered out by Data Layer Picker. Return true to exclude the data layer instance. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterDataLayerInstance, const UDataLayerInstance*);

class FDataLayerMode : public ISceneOutlinerMode
{
public:
	enum class EItemSortOrder : int32
	{
		WorldDataLayers = 0,
		DataLayer = 10,
		Actor = 20,
	};

	FDataLayerMode(const FDataLayerModeParams& Params);
	virtual ~FDataLayerMode();

	virtual void Rebuild() override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual void InitializeViewMenuExtender(TSharedPtr<FExtender> Extender) override;
	virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual bool IsInteractive() const override { return true; }
	virtual bool CanRename() const override { return true; }
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	virtual bool CanCustomizeToolbar() const { return true; }
	virtual bool ShowStatusBar() const override { return true; }
	virtual bool ShowViewButton() const override { return true; }
	virtual bool ShowFilterOptions() const override { return true; }
	virtual FText GetStatusText() const override;
	virtual FSlateColor GetStatusTextColor() const override { return FSlateColor::UseForeground(); }
	virtual FFolder::FRootObject GetRootObject() const override;

	virtual void SynchronizeSelection() override;
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) override;
	virtual void OnItemRemoved(FSceneOutlinerTreeItemPtr Item) override;
	virtual void OnItemPassesFilters(const ISceneOutlinerTreeItem& Item) override;
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;

	virtual bool CanSupportDragAndDrop() const { return true; }
	virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	virtual FReply OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const override; 

	void DeleteItems(const TArray<TWeakPtr<ISceneOutlinerTreeItem>>& Items);
	SDataLayerBrowser* GetDataLayerBrowser() const;

	void BuildWorldPickerMenu(FMenuBuilder& MenuBuilder);

protected:
	virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;

	/** Should Editor DataLayers be hidden */
	bool bHideEditorDataLayers;
	/** Should Runtime DataLayers be hidden */
	bool bHideRuntimeDataLayers;
	/** Should DataLayers actors be hidden */
	bool bHideDataLayerActors;
	/** Should unloaded actors be hidden */
	bool bHideUnloadedActors;
	/** Should show only selected actors */
	bool bShowOnlySelectedActors;
	/** Should highlight DataLayers containing selected actors */
	bool bHighlightSelectedDataLayers;
	/** Should level instance actors content be hidden */
	bool bHideLevelInstanceContent;

	/** Delegate to call when an item is picked */
	FOnSceneOutlinerItemPicked OnItemPicked;

private:
	/* Private Helpers */
	void RegisterContextMenu();
	void ChooseRepresentingWorld();
	void OnSelectWorld(TWeakObjectPtr<UWorld> World);
	bool IsWorldChecked(TWeakObjectPtr<UWorld> World) const;
	TArray<FDataLayerActorMoveElement> GetDataLayerActorPairsFromOperation(const FDragDropOperation& Operation) const;
	TArray<AActor*> GetActorsFromOperation(const FDragDropOperation& Operation, bool bOnlyFindFirst = false) const;
	TArray<UDataLayerInstance*> GetDataLayerInstancesFromOperation(const FDragDropOperation& Operation, bool bOnlyFindFirst = false) const;
	TArray<const UDataLayerAsset*> GetDataLayerAssetsFromOperation(const FDragDropOperation& InDragDrop, bool bOnlyFindFirst = false) const;
	TArray<UDataLayerInstance*> GetSelectedDataLayers(SSceneOutliner* InSceneOutliner) const;
	void SetParentDataLayer(const TArray<UDataLayerInstance*> DataLayers, UDataLayerInstance* ParentDataLayer) const;
	void OnLevelSelectionChanged(UObject* Obj);
	void OnPostLoadMapWithWorld(UWorld* World);
	static void CreateDataLayerPicker(UToolMenu* InMenu, FOnDataLayerInstancePicked OnDataLayerInstancePicked, FOnShouldFilterDataLayerInstance OnShouldFilterDataLayerInstance, bool bInShowRoot = false);
	bool ShouldExpandDataLayer(const UDataLayerInstance* DataLayer) const;
	bool ContainsSelectedChildDataLayer(const UDataLayerInstance* DataLayer) const;
	void DeleteDataLayers(const TArray<UDataLayerInstance*>& InDataLayersToDelete);
	void RefreshSelection();
	void CacheSelectedItems(const FSceneOutlinerItemSelection& Selection);
	UWorld* GetOwningWorld() const;
	AWorldDataLayers* GetOwningWorldAWorldDataLayers() const;
	AWorldDataLayers* GetWorldDataLayersFromTreeItem(const ISceneOutlinerTreeItem& TreeItem) const;
	UDataLayerInstance* GetDataLayerInstanceFromTreeItem(const ISceneOutlinerTreeItem& TreeItem) const;
	FSceneOutlinerDragValidationInfo ValidateActorDrop(const ISceneOutlinerTreeItem& DropTarget, TArray<AActor*> PayloadActors, bool bMoveOperation = false) const;
	FSceneOutlinerDragValidationInfo ValidateDataLayerAssetDrop(const ISceneOutlinerTreeItem& DropTarget, const TArray<const UDataLayerAsset*>& DataLayerAssetsToDrop, bool bIsMovingDataLayerInstances = false) const;
	bool CanReferenceDataLayerAssets(const AWorldDataLayers* InWorldDataLayers, const TArray<const UDataLayerAsset*>& InReferencedDataLayerAssets, FText* OutFailureReason = nullptr) const;
	void OnDataLayerAssetDropped(const TArray<const UDataLayerAsset*>& DroppedDataLayerAsset, ISceneOutlinerTreeItem& DropTarget) const;

	/** Filter factories */
	static TSharedRef<FSceneOutlinerFilter> CreateShowOnlySelectedActorsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideEditorDataLayersFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideRuntimeDataLayersFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideDataLayerActorsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideUnloadedActorsFilter();
	static TSharedRef<FSceneOutlinerFilter> CreateHideLevelInstancesFilter();

private:

	/** Delegate to handle "Find in Content Browser" context menu option */
	void FindInContentBrowser();
	/** Delegate to handle enabling the "Find in Content Browser" context menu option */
	bool CanFindInContentBrowser() const;
	/** The world which we are currently representing */
	TWeakObjectPtr<UWorld> RepresentingWorld;
	/** The world which the user manually selected */
	TWeakObjectPtr<UWorld> UserChosenWorld;
	/** The DataLayer browser */
	SDataLayerBrowser* DataLayerBrowser;
	/** The DataLayerEditorSubsystem */
	UDataLayerEditorSubsystem* DataLayerEditorSubsystem;
	/** If this mode was created to display a specific world, don't allow it to be reassigned */
	const TWeakObjectPtr<UWorld> SpecifiedWorldToDisplay;
	/** Number of datalayers which have passed through the filters */
	uint32 FilteredDataLayerCount = 0;
	/** List of datalayers which passed the regular filters and may or may not have passed the search filter */
	TSet<TWeakObjectPtr<UDataLayerInstance>> ApplicableDataLayers;
	/** The path at which the "Pick A Data Layer Asset" will be opened*/
	mutable FString PickDataLayerDialogPath;
	/** Command list */
	TSharedPtr<FUICommandList> Commands;

	TSet<TWeakObjectPtr<const UDataLayerInstance>> SelectedDataLayersSet;
	typedef TPair<TWeakObjectPtr<const UDataLayerInstance>, TWeakObjectPtr<const AActor>> FSelectedDataLayerActor;
	TSet<FSelectedDataLayerActor> SelectedDataLayerActors;
};

class FDataLayerPickingMode : public FDataLayerMode
{
public:
	FDataLayerPickingMode(const FDataLayerModeParams& Params);
	virtual TSharedPtr<SWidget> CreateContextMenu() override { return nullptr; }
	virtual bool ShowStatusBar() const override { return false; }
	virtual bool ShowViewButton() const override { return false; }
	virtual bool ShowFilterOptions() const override { return false; }
	virtual bool SupportsKeyboardFocus() const override { return false; }
	virtual bool CanRename() const override { return false; }
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override { return false; }
	virtual bool CanCustomizeToolbar() const { return false; }
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override {}
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override { return FReply::Unhandled(); }
	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	virtual void SynchronizeSelection() override {}

	static TSharedRef<SWidget> CreateDataLayerPickerWidget(FOnDataLayerInstancePicked OnDataLayerInstancePicked, FOnShouldFilterDataLayerInstance OnShouldFilterDataLayerInstance = FOnShouldFilterDataLayerInstance());
};
