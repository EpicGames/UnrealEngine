// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "DataLayer/DataLayerEditorSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "ScopedTransaction.h"
#include "IObjectNameEditSink.h"

#define LOCTEXT_NAMESPACE "DataLayer"

class FDataLayerNameEditSink : public UE::EditorWidgets::IObjectNameEditSink
{
	virtual UClass* GetSupportedClass() const override
	{
		return UDataLayerInstance::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		UDataLayerInstance* DataLayerInstance = CastChecked<UDataLayerInstance>(Object);
		if (DataLayerInstance->CanEditDataLayerShortName())
		{
			return FText::Format(FText::FromString("{0}"), FText::FromString(DataLayerInstance->GetDataLayerShortName()));
		}
		
		UDataLayerInstanceWithAsset* DataLayerWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance);
		if (DataLayerWithAsset && !DataLayerWithAsset->GetAsset())
		{
			return FText::FromString(TEXT("Unknown"));
		}
		return FText::Format(FText::FromString("{0} ({1})"), FText::FromString(DataLayerInstance->GetDataLayerShortName()), FText::FromString(DataLayerInstance->GetDataLayerFullName()));
	}

	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const override
	{
		UDataLayerInstance* DataLayerInstance = CastChecked<UDataLayerInstance>(Object);
		return !DataLayerInstance->CanEditDataLayerShortName() || DataLayerInstance->IsReadOnly();
	};

	virtual bool SetObjectDisplayName(UObject* Object, FString DisplayName) override
	{
		UDataLayerInstance* DataLayerInstance = CastChecked<UDataLayerInstance>(Object);
		if(DataLayerInstance->CanEditDataLayerShortName())
		{
			if (!DisplayName.Equals(DataLayerInstance->GetDataLayerShortName(), ESearchCase::CaseSensitive))
			{
				const FScopedTransaction Transaction(LOCTEXT("DataLayerNameEditSinkRenameDataLayerTransaction", "Rename Data Layer"));

				return UDataLayerEditorSubsystem::Get()->SetDataLayerShortName(DataLayerInstance, DisplayName);
			}
		}
		
		return false;
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		return IsObjectDisplayNameReadOnly(Object) ? LOCTEXT("NonEditableDataLayerLabel_TooltipFmt", "Data Layer Name") : FText::Format(LOCTEXT("EditableDataLayerLabel_TooltipFmt", "Rename the selected {0}"), FText::FromString(Object->GetClass()->GetName()));
	}
};

#undef LOCTEXT_NAMESPACE