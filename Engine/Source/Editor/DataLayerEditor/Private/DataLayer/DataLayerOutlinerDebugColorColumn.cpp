// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerOutlinerDebugColorColumn.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "DataLayerTreeItem.h"
#include "SceneOutlinerPublicTypes.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "DataLayer"

FName FDataLayerOutlinerDebugColorColumn::GetID()
{
	static FName DataLayeDebugColor("Debug Color");
	return DataLayeDebugColor;
}

SHeaderRow::FColumn::FArguments FDataLayerOutlinerDebugColorColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(20.f)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		[
			SNew(SSpacer)
		];
}

const TSharedRef<SWidget> FDataLayerOutlinerDebugColorColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (TreeItem->IsA<FDataLayerTreeItem>())
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity_Lambda([TreeItem]()
				{
					FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>();
					UDataLayerInstance* DataLayerInstance = DataLayerTreeItem ? DataLayerTreeItem->GetDataLayer() : nullptr;
					return DataLayerInstance ? DataLayerInstance->GetDebugColor() : FColor::Black;
				})
				.Image(FAppStyle::Get().GetBrush("Level.ColorIcon"))
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE