// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerOutlinerDeleteButtonColumn.h"

#include "Containers/Array.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayerActorTreeItem.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISceneOutlinerTreeItem.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "SceneOutlinerPublicTypes.h"
#include "Styling/AppStyle.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class AActor;
class SWidget;
template <typename ItemType> class STableRow;

#define LOCTEXT_NAMESPACE "DataLayer"

FName FDataLayerOutlinerDeleteButtonColumn::GetID()
{
	static FName DataLayeDeleteButton("Remove Actor");
	return DataLayeDeleteButton;
}

SHeaderRow::FColumn::FArguments FDataLayerOutlinerDeleteButtonColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
	.FixedWidth(40.f)
	.DefaultTooltip(FText::FromName(GetColumnID()))
	[
		SNew(SSpacer)
	];
}

const TSharedRef<SWidget> FDataLayerOutlinerDeleteButtonColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FDataLayerActorTreeItem* DataLayerActorItem = TreeItem->CastTo<FDataLayerActorTreeItem>())
	{
		return SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "DataLayerBrowserButton")
			.ContentPadding(0)
			.Visibility_Lambda([this, TreeItem, DataLayerActorItem]()
			{
				AActor* Actor = DataLayerActorItem->GetActor();
				const UDataLayerInstance* DataLayerInstance = DataLayerActorItem->GetDataLayer();
				return (Actor && Actor->IsUserManaged() && DataLayerInstance && DataLayerInstance->CanRemoveActor(Actor) && DataLayerInstance->CanUserRemoveActors() && TreeItem->CanInteract()) ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.OnClicked_Lambda([this, TreeItem, DataLayerActorItem]()
			{
				AActor* Actor = DataLayerActorItem->GetActor();
				const UDataLayerInstance* DataLayerInstance = DataLayerActorItem->GetDataLayer();
				if (Actor && DataLayerInstance)
				{
					UDataLayerEditorSubsystem* DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get();
					if (auto SceneOutliner = WeakSceneOutliner.IsValid() ? WeakSceneOutliner.Pin() : nullptr)
					{
						const auto& Tree = SceneOutliner->GetTree();
						if (SceneOutliner->GetSharedData().CustomDelete.IsBound())
						{
							TArray<TWeakPtr<ISceneOutlinerTreeItem>> SelectedItems;
							if (Tree.IsItemSelected(TreeItem))
							{
								for (auto& SelectedItem : Tree.GetSelectedItems())
								{
									if (FDataLayerActorTreeItem* SelectedDataLayerActorTreeItem = SelectedItem->CastTo<FDataLayerActorTreeItem>())
									{
										SelectedItems.Add(SelectedDataLayerActorTreeItem->AsShared());
									}
								}
							}
							else
							{
								SelectedItems.Add(TreeItem);
							}
							SceneOutliner->GetSharedData().CustomDelete.Execute(SelectedItems);
						}
					}
				}
				return FReply::Handled();
			})
			.ToolTipText(LOCTEXT("RemoveFromDataLayerButtonText", "Remove from Data Layer"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("DataLayerBrowser.Actor.RemoveFromDataLayer")))
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE