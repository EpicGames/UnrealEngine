// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerOutlinerHasErrorColumn.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "DataLayerTreeItem.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Text.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationTokenizedMessageErrorHandler.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DataLayer"

namespace
{
	FName DataLayerOutlinerHasErrors("Data Layer Has Errors");
}

FName FDataLayerOutlinerHasErrorsColumn::GetID()
{
	return ::DataLayerOutlinerHasErrors;
}

SHeaderRow::FColumn::FArguments FDataLayerOutlinerHasErrorsColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(FText::FromString(TEXT("")))
		];
}

const TSharedRef<SWidget> FDataLayerOutlinerHasErrorsColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>())
	{
		if (UDataLayerInstance* DataLayerInstance = DataLayerTreeItem->GetDataLayer())
		{
			FTokenizedMessageAccumulatorErrorHandler ErrorHandler;
			if (!DataLayerInstance->Validate(&ErrorHandler))
			{
				auto MakeErrorMessage = [](const TArray<TSharedRef<FTokenizedMessage>>& ErrorMessages)
				{
					TArray<FText> ErrorsAsText;
					Algo::Transform(ErrorMessages, ErrorsAsText, [](const TSharedRef<FTokenizedMessage>& Error) { return Error->ToText(); });

					return FText::Join(FText::FromString("\n"), ErrorsAsText);
				};

				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
						.ToolTipText(MakeErrorMessage(ErrorHandler.GetErrorMessages()))
					];
			}
			else
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center);
			}
		}
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE