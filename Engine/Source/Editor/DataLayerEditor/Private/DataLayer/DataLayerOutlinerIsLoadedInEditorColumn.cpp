// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerOutlinerIsLoadedInEditorColumn.h"

#include "Containers/Array.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayerTreeItem.h"
#include "Engine/World.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STreeView.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DataLayer"

FName FDataLayerOutlinerIsLoadedInEditorColumn::GetID()
{
	static FName DataLayerIsLoadedInEditor("Data Layer Loaded In Editor");
	return DataLayerIsLoadedInEditor;
}

SHeaderRow::FColumn::FArguments FDataLayerOutlinerIsLoadedInEditorColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("DataLayerLoadedInEditorTooltip", "Data Layer Loaded In Editor"))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("DataLayer.LoadedInEditor")))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

const TSharedRef<SWidget> FDataLayerOutlinerIsLoadedInEditorColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (TreeItem->IsA<FDataLayerTreeItem>())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 0, 0, 0)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsEnabled_Lambda([TreeItem]()
				{
					FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>();
					const UDataLayerInstance* DataLayer = DataLayerTreeItem->GetDataLayer();
					const UDataLayerInstance* ParentDataLayer = DataLayer ? DataLayer->GetParent() : nullptr;
					const bool bIsParentLoaded = ParentDataLayer ? ParentDataLayer->IsEffectiveLoadedInEditor() : true;
					return bIsParentLoaded && DataLayer && DataLayer->GetWorld() && !DataLayer->GetWorld()->IsPlayInEditor();
				})
				.Visibility_Lambda([TreeItem]()
				{
					FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>();
					const UDataLayerInstance* DataLayerInstance = DataLayerTreeItem->GetDataLayer();
					const AWorldDataLayers* OuterWorldDataLayers = DataLayerInstance ? DataLayerInstance->GetDirectOuterWorldDataLayers() : nullptr;
					const bool bIsSubWorldDataLayers = OuterWorldDataLayers && OuterWorldDataLayers->IsSubWorldDataLayers();
					return DataLayerInstance && !DataLayerInstance->IsReadOnly() && !bIsSubWorldDataLayers ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked_Lambda([TreeItem]()
				{
					FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>();
					UDataLayerInstance* DataLayer = DataLayerTreeItem->GetDataLayer();
					return DataLayer && DataLayer->IsEffectiveLoadedInEditor() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this, TreeItem](ECheckBoxState NewState)
				{
					bool bNewState = (NewState == ECheckBoxState::Checked);
					FDataLayerTreeItem* DataLayerTreeItem = TreeItem->CastTo<FDataLayerTreeItem>();
					if (UDataLayerInstance* DataLayer = DataLayerTreeItem->GetDataLayer())
					{
						UWorld* World = DataLayer->GetWorld();
						UDataLayerEditorSubsystem* DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get();
						const auto& Tree = WeakSceneOutliner.Pin()->GetTree();
						if (Tree.IsItemSelected(TreeItem))
						{
							// Toggle IsLoadedInEditor flag of selected DataLayers to the same state as the given DataLayer
							const bool bIsLoadedInEditor = DataLayer->IsLoadedInEditor();

							TArray<UDataLayerInstance*> AllSelectedDataLayers;
							for (auto& SelectedItem : Tree.GetSelectedItems())
							{
								FDataLayerTreeItem* SelectedDataLayerTreeItem = SelectedItem->CastTo<FDataLayerTreeItem>();
								UDataLayerInstance* SelectedDataLayer = SelectedDataLayerTreeItem ? SelectedDataLayerTreeItem->GetDataLayer() : nullptr;
								if (SelectedDataLayer && SelectedDataLayer->IsLoadedInEditor() == bIsLoadedInEditor)
								{
									AllSelectedDataLayers.Add(SelectedDataLayer);
								}
							}

							const FScopedTransaction Transaction(LOCTEXT("ToggleDataLayersIsLoadedInEditor", "Toggle Data Layers Dynamically Loaded In Editor Flag"));
							DataLayerEditorSubsystem->ToggleDataLayersIsLoadedInEditor(AllSelectedDataLayers, /*bIsFromUserChange*/true);
						}
						else
						{
							const FScopedTransaction Transaction(LOCTEXT("ToggleDataLayerIsLoadedInEditor", "Toggle Data Layer Dynamically Loaded In Editor Flag"));
							DataLayerEditorSubsystem->ToggleDataLayerIsLoadedInEditor(DataLayer, /*bIsFromUserChange*/true);
						}
					}
				})
				.ToolTipText(LOCTEXT("IsLoadedInEditorCheckBoxToolTip", "Toggle Loaded In Editor Flag"))
				.HAlign(HAlign_Center)
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE