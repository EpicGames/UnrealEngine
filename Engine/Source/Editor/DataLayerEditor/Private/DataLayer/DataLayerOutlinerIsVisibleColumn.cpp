// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerOutlinerIsVisibleColumn.h"

#include "DataLayerTreeItem.h"
#include "Engine/World.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerTreeItem.h"
#include "Math/Color.h"
#include "Math/ColorList.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "DataLayer"

FName FDataLayerOutlinerIsVisibleColumn::GetID()
{
	static FName DataLayerOutlinerIsVisible("Data Layer Visibility");
	return DataLayerOutlinerIsVisible;
}

/** Widget responsible for managing the visibility for a single item */
class SDataLayerVisibilityWidget : public SVisibilityWidget
{
protected:

	virtual bool IsEnabled() const override
	{
		auto TreeItem = WeakTreeItem.Pin();
		if (FDataLayerTreeItem* DataLayerTreeItem = TreeItem.IsValid() ? TreeItem->CastTo<FDataLayerTreeItem>() : nullptr)
		{
			const UDataLayerInstance* DataLayer = DataLayerTreeItem->GetDataLayer();
			const UDataLayerInstance* ParentDataLayer = DataLayer ? DataLayer->GetParent() : nullptr;
			const bool bIsParentVisible = ParentDataLayer ? ParentDataLayer->IsEffectiveVisible() : true;
			return bIsParentVisible && DataLayer && DataLayer->GetWorld() && !DataLayer->GetWorld()->IsPlayInEditor() && DataLayer->IsEffectiveLoadedInEditor();
		}
		return false;
	}

	virtual const FSlateBrush* GetBrush() const override
	{
		bool bIsEffectiveVisible = false;
		auto TreeItem = WeakTreeItem.Pin();
		if (FDataLayerTreeItem* DataLayerTreeItem = TreeItem.IsValid() ? TreeItem->CastTo<FDataLayerTreeItem>() : nullptr)
		{
			UDataLayerInstance* DataLayer = DataLayerTreeItem->GetDataLayer();
			bIsEffectiveVisible = DataLayer && DataLayer->IsEffectiveVisible();
		}

		if (bIsEffectiveVisible)
		{
			return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
		}
		else
		{
			return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
		}
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		if (IsEnabled())
		{
			auto Outliner = WeakOutliner.Pin();
			auto TreeItem = WeakTreeItem.Pin();
			const bool bIsSelected = Outliner->GetTree().IsItemSelected(TreeItem.ToSharedRef());
			if (IsVisible() && !Row->IsHovered() && !bIsSelected)
			{
				return FLinearColor::Transparent;
			}
			return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
		}
		else
		{
			return FLinearColor(FColorList::DimGrey);
		}
	}

	virtual bool ShouldPropagateVisibilityChangeOnChildren() const { return false; }
};

const TSharedRef<SWidget> FDataLayerOutlinerIsVisibleColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (TreeItem->ShouldShowVisibilityState())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SDataLayerVisibilityWidget, SharedThis(this), WeakOutliner, TreeItem, &Row)
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE