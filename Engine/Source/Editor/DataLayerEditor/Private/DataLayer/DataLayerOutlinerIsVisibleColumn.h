// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "SceneOutlinerGutter.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISceneOutliner;
class SWidget;
template<typename ItemType> class STableRow;

class FDataLayerOutlinerIsVisibleColumn : public FSceneOutlinerGutter
{
public:
	FDataLayerOutlinerIsVisibleColumn(ISceneOutliner& SceneOutliner) : FSceneOutlinerGutter(SceneOutliner) {}
	virtual ~FDataLayerOutlinerIsVisibleColumn() = default;
	static FName GetID();

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override { return GetID(); }
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////
};
