// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"

class FDataLayerDragDropOp;
class FDetailWidgetRow;
class FDragDropEvent;
class FDragDropOperation;
class FReply;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class UDataLayerInstance;
struct EVisibility;
struct FGeometry;
struct FSlateBrush;
struct FSlateColor;

struct UE_DEPRECATED(5.8, "The ActorDataLayer property on AActor is no longer supported") FDataLayerPropertyTypeCustomization;

struct FDataLayerPropertyTypeCustomization : public IPropertyTypeCustomization
{
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
};
