// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/DataLayerPropertyTypeCustomizationHelper.h"

#include "DataLayerMode.h"
#include "Delegates/Delegate.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DataLayer"

TSharedRef<SWidget> FDataLayerPropertyTypeCustomizationHelper::CreateDataLayerMenu(TFunction<void(const UDataLayerInstance* DataLayerInstance)> OnDataLayerSelectedFunction, TFunction<bool(const UDataLayerInstance*)> OnShouldFilterDataLayerInstanceFunction)
{
	return FDataLayerPickingMode::CreateDataLayerPickerWidget(
		FOnDataLayerInstancePicked::CreateLambda([OnDataLayerSelectedFunction](UDataLayerInstance* TargetDataLayerInstance)
		{
			OnDataLayerSelectedFunction(TargetDataLayerInstance);
		}),
		FOnShouldFilterDataLayerInstance::CreateLambda([OnShouldFilterDataLayerInstanceFunction](const UDataLayerInstance* DataLayerInstance)
		{
			return OnShouldFilterDataLayerInstanceFunction(DataLayerInstance);
		}));
}

#undef LOCTEXT_NAMESPACE