// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerTreeItem.h"

#include "DataLayer/DataLayerEditorSubsystem.h"
#include "ISceneOutlinerTreeItem.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "ScopedTransaction.h"

template <typename ItemType> class STableRow;

#define LOCTEXT_NAMESPACE "DataLayer"

const FSceneOutlinerTreeItemType FDataLayerTreeItem::Type(&ISceneOutlinerTreeItem::Type);

struct SDataLayerTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDataLayerTreeLabel) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FDataLayerTreeItem& DataLayerItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow);

private:

	FSlateFontInfo GetDisplayNameFont() const;
	FText GetDisplayText() const;
	FText GetTooltipText() const;
	FText GetTypeText() const;
	EVisibility GetTypeTextVisibility() const;
	const FSlateBrush* GetIcon() const;
	FText GetIconTooltip() const;
	FSlateColor GetIconColor() const;
	FSlateColor GetForegroundColor() const;
	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);
	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);
	bool ShouldBeHighlighted() const;
	bool ShouldBeItalic() const;
	bool IsInActorEditorContext() const;
	void OnEnterEditingMode();
	void OnExitEditingMode();

	TWeakPtr<FDataLayerTreeItem> TreeItemPtr;
	TWeakObjectPtr<UDataLayerInstance> DataLayerPtr;
	TAttribute<FText> HighlightText;
	bool bInEditingMode;
};

FDataLayerTreeItem::FDataLayerTreeItem(UDataLayerInstance* InDataLayerInstance)
	: ISceneOutlinerTreeItem(Type)
	, DataLayerInstance(InDataLayerInstance)
	, ID(InDataLayerInstance)
	, bIsHighlighedtIfSelected(false)
{
	Flags.bIsExpanded = false;
}

FString FDataLayerTreeItem::GetDisplayString() const
{
	const UDataLayerInstance* DataLayerInstancePtr = DataLayerInstance.Get();
	return DataLayerInstancePtr ? DataLayerInstancePtr->GetDataLayerShortName() : LOCTEXT("DataLayerForMissingDataLayer", "(Deleted Data Layer)").ToString();
}

bool FDataLayerTreeItem::GetVisibility() const
{
	const UDataLayerInstance* DataLayerInstancePtr = DataLayerInstance.Get();
	return DataLayerInstancePtr && DataLayerInstancePtr->IsVisible();
}

bool FDataLayerTreeItem::ShouldShowVisibilityState() const
{
	const UDataLayerInstance* DataLayerInstancePtr = DataLayerInstance.Get();
	const AWorldDataLayers* OuterWorldDataLayers = DataLayerInstancePtr ? DataLayerInstancePtr->GetDirectOuterWorldDataLayers() : nullptr;
	const bool bIsSubWorldDataLayers = OuterWorldDataLayers && OuterWorldDataLayers->IsSubWorldDataLayers();
	return DataLayerInstancePtr && !DataLayerInstancePtr->IsReadOnly() && !bIsSubWorldDataLayers;
}

bool FDataLayerTreeItem::CanInteract() const
{
	return true;
}

TSharedRef<SWidget> FDataLayerTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SDataLayerTreeLabel, *this, Outliner, InRow);
}

void FDataLayerTreeItem::OnVisibilityChanged(const bool bNewVisibility)
{
	if (UDataLayerInstance* DataLayerInstancePtr = DataLayerInstance.Get())
	{
		UDataLayerEditorSubsystem::Get()->SetDataLayerVisibility(DataLayerInstancePtr, bNewVisibility);
	}
}

bool FDataLayerTreeItem::ShouldBeHighlighted() const
{
	if (bIsHighlighedtIfSelected)
	{
		if (UDataLayerInstance* DataLayerInstancePtr = DataLayerInstance.Get())
		{
			return UDataLayerEditorSubsystem::Get()->DoesDataLayerContainSelectedActors(DataLayerInstancePtr);
		}
	}
	return false;
}

void SDataLayerTreeLabel::Construct(const FArguments& InArgs, FDataLayerTreeItem& DataLayerItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
	TreeItemPtr = StaticCastSharedRef<FDataLayerTreeItem>(DataLayerItem.AsShared());
	DataLayerPtr = DataLayerItem.GetDataLayer();
	HighlightText = SceneOutliner.GetFilterHighlightText();
	bInEditingMode = false;

	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;
	check(!InlineTextBlock.IsValid()); // This check is to make sure we aren't encountering the compiler bug which doesn't properly construct 'InlineTextBlock'

	auto MainContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
			.Font(this, &SDataLayerTreeLabel::GetDisplayNameFont)
			.Text(this, &SDataLayerTreeLabel::GetDisplayText)
			.ToolTipText(this, &SDataLayerTreeLabel::GetTooltipText)
			.HighlightText(HighlightText)
			.ColorAndOpacity(this, &SDataLayerTreeLabel::GetForegroundColor)
			.OnTextCommitted(this, &SDataLayerTreeLabel::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SDataLayerTreeLabel::OnVerifyItemLabelChanged)
			.OnEnterEditingMode(this, &SDataLayerTreeLabel::OnEnterEditingMode)
			.OnExitEditingMode(this, &SDataLayerTreeLabel::OnExitEditingMode)
			.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelectedExclusively))
			.IsReadOnly_Lambda([Item = DataLayerItem.AsShared(), this]()
			{
				return DataLayerPtr == nullptr || !DataLayerPtr->CanEditDataLayerShortName() || !CanExecuteRenameRequest(Item.Get());
			})
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f, 0.f, 3.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(this, &SDataLayerTreeLabel::GetTypeText)
			.Visibility(this, &SDataLayerTreeLabel::GetTypeTextVisibility)
			.HighlightText(HighlightText)
		];

	if (WeakSceneOutliner.Pin()->GetMode()->IsInteractive())
	{
		DataLayerItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
		[
			SNew(SBox)
			.WidthOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
			.HeightOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
			[
				SNew(SImage)
				.Image(this, &SDataLayerTreeLabel::GetIcon)
				.ToolTipText(this, &SDataLayerTreeLabel::GetIconTooltip)
				.ColorAndOpacity(this, &SDataLayerTreeLabel::GetIconColor)
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f)
		[
			MainContent
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Visibility_Lambda([this] { return (DataLayerPtr.IsValid() && DataLayerPtr->IsReadOnly() && DataLayerPtr->GetWorld() && !DataLayerPtr->GetWorld()->IsPlayInEditor()) ? EVisibility::Visible : EVisibility::Collapsed; })
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush(TEXT("PropertyWindow.Locked")))
			.ToolTipText_Lambda([this]
			{
				if (const UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get())
				{
					FText Reason;
					if (DataLayerInstance->IsReadOnly(&Reason))
					{
						return FText::Format(LOCTEXT("ReadOnlyDataLayerInstance", "{0}"), Reason);
					}
				}
				return FText::GetEmpty();
			})
		]
	];
}

bool SDataLayerTreeLabel::ShouldBeHighlighted() const
{
	const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin();
	FDataLayerTreeItem* DataLayerTreeItem = TreeItem ? TreeItem->CastTo<FDataLayerTreeItem>() : nullptr;
	return DataLayerTreeItem && DataLayerTreeItem->ShouldBeHighlighted();
}

bool SDataLayerTreeLabel::ShouldBeItalic() const
{
	if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerPtr.Get()))
	{
		return !DataLayerInstanceWithAsset->GetAsset();
	}

	return false;
}

bool SDataLayerTreeLabel::IsInActorEditorContext() const
{
	const UDataLayerInstance* DataLayer = DataLayerPtr.Get();
	return DataLayer && DataLayer->IsInActorEditorContext();
}

FSlateFontInfo SDataLayerTreeLabel::GetDisplayNameFont() const
{
	if (ShouldBeHighlighted())
	{
		return FAppStyle::Get().GetFontStyle("DataLayerBrowser.LabelFontBold");
	}
	else if (ShouldBeItalic())
	{
		return FAppStyle::Get().GetFontStyle("PropertyWindow.ItalicFont");
	}

	return FAppStyle::Get().GetFontStyle("DataLayerBrowser.LabelFont");
}

FText SDataLayerTreeLabel::GetDisplayText() const
{
	const UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get();
	bool bIsDataLayerActive = false;
	FText SuffixText = FText::GetEmpty();
	if (!bInEditingMode)
	{
		if (DataLayerInstance && DataLayerInstance->IsRuntime() && DataLayerInstance->GetWorld() && DataLayerInstance->GetWorld()->IsPlayInEditor())
		{
			SuffixText = FText::Format(LOCTEXT("DataLayerRuntimeState", " ({0})"), FTextStringHelper::CreateFromBuffer(GetDataLayerRuntimeStateName(DataLayerInstance->GetEffectiveRuntimeState())));
		}
		else if (IsInActorEditorContext())
		{
			SuffixText = FText(LOCTEXT("IsCurrentSuffix", " (Current)"));
		}
	}

	if (DataLayerInstance)
	{
		if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
		{
			if (!DataLayerInstanceWithAsset->GetAsset())
			{
				return LOCTEXT("DataLayerLabelUnknown", "Unknown");
			}
		}

		return FText::Format(LOCTEXT("DataLayerDisplayText", "{0}{1}"), FText::FromString(DataLayerInstance->GetDataLayerShortName()), SuffixText);
	}

	static const FText DataLayerDeleted = LOCTEXT("DataLayerLabelForMissingDataLayer", "(Deleted Data Layer)");
	return DataLayerDeleted;
}

FText SDataLayerTreeLabel::GetTooltipText() const
{
	if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
	{
		FText Description = IsInActorEditorContext() ? LOCTEXT("DataLayerIsCurrentDescription", "This Data Layer is part of Current Data Layers. New actors will attempt to be added to this Data Layer.") : FText::GetEmpty();
		return FText::Format(LOCTEXT("DataLayerTooltipText", "{0}\n{1}"), FText::FromString(TreeItem->GetDisplayString()), Description);
	}

	return FText();
}

FText SDataLayerTreeLabel::GetTypeText() const
{
	const UDataLayerInstance* DataLayer = DataLayerPtr.Get();
	return DataLayer ? FText::FromName(DataLayer->GetClass()->GetFName()) : FText();
}

EVisibility SDataLayerTreeLabel::GetTypeTextVisibility() const
{
	return HighlightText.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

const FSlateBrush* SDataLayerTreeLabel::GetIcon() const
{
	const UDataLayerInstance* DataLayer = DataLayerPtr.Get();
	if (DataLayer && WeakSceneOutliner.IsValid())
	{
		const TCHAR* IconName = DataLayer->GetDataLayerIconName();
		if (const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(IconName))
		{
			return CachedBrush;
		}

		const FSlateBrush* FoundSlateBrush = FAppStyle::GetBrush(IconName);
		WeakSceneOutliner.Pin()->CacheIconForClass(IconName, FoundSlateBrush);
		return FoundSlateBrush;
	}
	return nullptr;
}

FText SDataLayerTreeLabel::GetIconTooltip() const
{
	if (const UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get())
	{
		if (DataLayerInstance->IsA<UExternalDataLayerInstance>())
		{
			return FText(LOCTEXT("ExternalDataLayer", "External Data Layer"));
		}
		if (DataLayerInstance->IsRuntime())
		{
			return FText(LOCTEXT("RuntimeDataLayer", "Runtime Data Layer"));
		}
		return FText(LOCTEXT("EditorDataLayer", "Editor Data Layer"));
	}
	return FText();
}

FSlateColor SDataLayerTreeLabel::GetIconColor() const
{
	const UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get();
	const UExternalDataLayerInstance* ExternalDataLayerInstance = DataLayerInstance ? DataLayerInstance->GetRootExternalDataLayerInstance() : nullptr;
	return ExternalDataLayerInstance ? UExternalDataLayerAsset::EditorUXColor : FSlateColor::UseForeground();
}

FSlateColor SDataLayerTreeLabel::GetForegroundColor() const
{
	if (TOptional<FLinearColor> BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
	{
		return BaseColor.GetValue();
	}

	const UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get();
	if (!DataLayerInstance || !DataLayerInstance->GetWorld())
	{
		return FLinearColor(0.2f, 0.2f, 0.25f);
	}
	if (DataLayerInstance->GetWorld()->IsPlayInEditor())
	{
		if (DataLayerInstance->IsRuntime())
		{
			EDataLayerRuntimeState State = DataLayerInstance->GetEffectiveRuntimeState();
			switch (State)
			{
			case EDataLayerRuntimeState::Activated:
				return FColorList::LimeGreen;
			case EDataLayerRuntimeState::Loaded:
				return FColorList::NeonBlue;
			case EDataLayerRuntimeState::Unloaded:
				return FColorList::DarkSlateGrey;
			}
		}
		else
		{
			return FSceneOutlinerCommonLabelData::DarkColor;
		}
	}
	else if (DataLayerInstance->IsReadOnly())
	{
		return FSceneOutlinerCommonLabelData::DarkColor;
	}
	if (IsInActorEditorContext())
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
	}
	return FSlateColor::UseForeground();
}

bool SDataLayerTreeLabel::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	if (InLabel.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("EmptyDataLayerLabel", "Data Layer must be given a name");
		return false;
	}

	UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get();
	if (!DataLayerInstance->CanEditDataLayerShortName())
	{
		OutErrorMessage = LOCTEXT("RenameFailed_NotPermittedOnDataLayer", "This Data Layer does not support renaming");
		return false;
	}

	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(DataLayerPtr.Get());
	check(DataLayerManager);

	TSet<UDataLayerInstance*> OutDataLayersWithShortName;
	if (FDataLayerUtils::FindDataLayerByShortName(DataLayerManager, InLabel.ToString(), OutDataLayersWithShortName))
	{
		if (!OutDataLayersWithShortName.Contains(DataLayerPtr.Get()))
		{
			OutErrorMessage = LOCTEXT("RenameFailed_AlreadyExists", "This Data Layer already exists");
			return false;
		}
	}

	return true;
}

void SDataLayerTreeLabel::OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	UDataLayerInstance* DataLayerInstance = DataLayerPtr.Get();
	check(DataLayerInstance->CanEditDataLayerShortName());
	if (!InLabel.ToString().Equals(DataLayerInstance->GetDataLayerShortName(), ESearchCase::CaseSensitive))
	{
		const FScopedTransaction Transaction(LOCTEXT("SceneOutlinerRenameDataLayerTransaction", "Rename Data Layer"));

		UDataLayerEditorSubsystem::Get()->SetDataLayerShortName(DataLayerInstance, InLabel.ToString());

		if (WeakSceneOutliner.IsValid())
		{
			WeakSceneOutliner.Pin()->SetKeyboardFocus();
		}
	}
}

void SDataLayerTreeLabel::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SDataLayerTreeLabel::OnExitEditingMode()
{
	bInEditingMode = false;
}


#undef LOCTEXT_NAMESPACE