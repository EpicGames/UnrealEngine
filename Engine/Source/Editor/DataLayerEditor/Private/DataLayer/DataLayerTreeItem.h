// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "ISceneOutlinerTreeItem.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class ISceneOutliner;
class SWidget;
template <typename ItemType> class STableRow;

struct FDataLayerTreeItem : ISceneOutlinerTreeItem
{
public:
	FDataLayerTreeItem(UDataLayerInstance* InDataLayerInstance);
	UDataLayerInstance* GetDataLayer() const { return DataLayerInstance.Get(); }

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return DataLayerInstance.IsValid(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return ID; }
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual bool HasVisibilityInfo() const override { return true; }
	virtual void OnVisibilityChanged(const bool bNewVisibility) override;
	virtual bool GetVisibility() const override;
	virtual bool ShouldShowVisibilityState() const override;
	/* End ISceneOutlinerTreeItem Implementation */

	bool ShouldBeHighlighted() const;
	void SetIsHighlightedIfSelected(bool bInIsHighlightedIfSelected) { bIsHighlighedtIfSelected = bInIsHighlightedIfSelected; }

	static const FSceneOutlinerTreeItemType Type;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const UDataLayerInstance*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const UDataLayerInstance*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(GetDataLayer());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(GetDataLayer());
	}

private:
	TWeakObjectPtr<UDataLayerInstance> DataLayerInstance;
	const FObjectKey ID;
	bool bIsHighlighedtIfSelected;
};