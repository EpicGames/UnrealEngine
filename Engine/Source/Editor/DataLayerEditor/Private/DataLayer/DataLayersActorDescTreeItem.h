// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "SceneOutlinerFwd.h"
#include "ActorDescTreeItem.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/SecureHash.h"
#include "UObject/ObjectKey.h"


//////////////////////////////////////////////////////////////////////////
// FDataLayerActorTreeItemData

class FWorldPartitionActorDesc;

struct FDataLayerActorDescTreeItemData
{
	FDataLayerActorDescTreeItemData(const FGuid& InActorGuid, UActorDescContainerInstance* InContainerInstance, UDataLayerInstance* InDataLayer)
		: ActorGuid(InActorGuid)
		, ContainerInstance(InContainerInstance)
		, DataLayer(InDataLayer)
	{}

	const FGuid& ActorGuid;
	UActorDescContainerInstance* const ContainerInstance;
	TWeakObjectPtr<UDataLayerInstance> DataLayer;
};

//////////////////////////////////////////////////////////////////////////
// FDataLayerActorTreeItem

struct FDataLayerActorDescTreeItem : public FActorDescTreeItem
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FFilterPredicate, const FWorldPartitionActorDescInstance* ActorDescInstance, const UDataLayerInstance* DataLayer);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FInteractivePredicate, const FWorldPartitionActorDescInstance* ActorDescInstance, const UDataLayerInstance* DataLayer);

	FDataLayerActorDescTreeItem(const FDataLayerActorDescTreeItemData& InData)
		: FActorDescTreeItem(InData.ActorGuid, InData.ContainerInstance)
		, DataLayer(InData.DataLayer)
		, IDDataLayerActorDesc(FDataLayerActorDescTreeItem::ComputeTreeItemID(InData.ActorGuid, InData.ContainerInstance, InData.DataLayer.Get()))
	{
		if (ActorDescHandle.IsValid())
		{
			UActorDescContainerInstance* ContainerInstance = ActorDescHandle->GetContainerInstance();
			UWorld* OwningWorld = ContainerInstance->GetOuterWorldPartition()->GetWorld();
			ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld);
			ULevel* Level = ContainerInstance->GetTypedOuter<UWorld>()->PersistentLevel;
			if (LevelInstanceSubsystem && Level && (Level != OwningWorld->GetCurrentLevel()))
			{
				DisplayString = LevelInstanceSubsystem->PrefixWithParentLevelInstanceActorLabels(DisplayString, Level);
			}
		}
	}

	UDataLayerInstance* GetDataLayer() const { return DataLayer.Get(); }
	
		static FSceneOutlinerTreeItemID ComputeTreeItemID(FGuid InActorGuid, UActorDescContainerInstance* InContainerInstance, const UDataLayerInstance* InDataLayer)
	{
		FObjectKey ContainerKey(InContainerInstance);
		FObjectKey DataLayerInstanceKey(InDataLayer);

		FArchiveMD5 Ar;
		Ar << InActorGuid << ContainerKey << DataLayerInstanceKey;

		return FSceneOutlinerTreeItemID(Ar.GetGuidFromHash());
	}

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(*ActorDescHandle, DataLayer.Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(*ActorDescHandle, DataLayer.Get());
	}

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return ActorDescHandle.IsValid() && DataLayer.IsValid(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return IDDataLayerActorDesc; }
	virtual bool ShouldShowVisibilityState() const { return false; }
	virtual bool HasVisibilityInfo() const override { return false; }
	virtual void OnVisibilityChanged(const bool bNewVisibility) override {}
	virtual bool GetVisibility() const override { return false; }
	/* End ISceneOutlinerTreeItem Implementation */

private:
	TWeakObjectPtr<UDataLayerInstance> DataLayer;
	const FSceneOutlinerTreeItemID IDDataLayerActorDesc;
};