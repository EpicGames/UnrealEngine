// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayer/ExternalDataLayerFactory.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "Settings/EditorExperimentalSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerFactory)

UExternalDataLayerFactory::UExternalDataLayerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UExternalDataLayerAsset::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

bool UExternalDataLayerFactory::CanCreateNew() const
{
	if (!GetDefault<UEditorExperimentalSettings>()->bEnableWorldPartitionExternalDataLayers)
	{
		return false;
	}
	return Super::CanCreateNew();
}

UObject* UExternalDataLayerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UDataLayerAsset* DataLayerAsset = NewObject<UExternalDataLayerAsset>(InParent, InName, Flags);
	DataLayerAsset->OnCreated();
	return DataLayerAsset;
}
