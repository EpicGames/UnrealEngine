// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataLayerBrowser.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayerActorTreeItem.h"
#include "DataLayerMode.h"
#include "DataLayerOutlinerDeleteButtonColumn.h"
#include "DataLayerOutlinerDebugColorColumn.h"
#include "DataLayerOutlinerHasErrorColumn.h"
#include "DataLayerOutlinerIsLoadedInEditorColumn.h"
#include "DataLayerOutlinerIsVisibleColumn.h"
#include "DataLayerTreeItem.h"
#include "DataLayersActorDescTreeItem.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailsViewArgs.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SDataLayerOutliner.h"
#include "SSceneOutliner.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerTextInfoColumn.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "Editor.h"

class ISceneOutliner;
class UObject;

#define LOCTEXT_NAMESPACE "DataLayer"

void SDataLayerBrowser::Construct(const FArguments& InArgs)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	Args.bAllowSearch = true;
	Args.bAllowFavoriteSystem = true;
	Args.bHideSelectionTip = true;
	Args.bShowObjectLabel = true;
	Args.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	Args.ColumnWidth = 0.5f;
	DetailsWidget = PropertyModule.CreateDetailView(Args);
	DetailsWidget->SetVisibility(EVisibility::Visible);

	//////////////////////////////////////////////////////////////////////////
	//	DataLayer Contents Header
	SAssignNew(DataLayerContentsHeader, SBorder)
	.BorderImage(FAppStyle::GetBrush("DataLayerBrowser.DataLayerContentsQuickbarBackground"))
	.Visibility(EVisibility::Visible);

	//////////////////////////////////////////////////////////////////////////
	//	DataLayer Contents Section

	FGetTextForItem InternalNameInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FDataLayerTreeItem* DataLayerItem = Item.CastTo<FDataLayerTreeItem>())
		{
			if (const UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer())
			{
				return DataLayerInstance->GetFName().ToString();
			}
		}
		else if (const FDataLayerActorTreeItem* DataLayerActorTreeItem = Item.CastTo<FDataLayerActorTreeItem>())
		{
			if (const AActor* Actor = DataLayerActorTreeItem->GetActor())
			{
				return Actor->GetFName().ToString();
			}
		}
		else if (const FDataLayerActorDescTreeItem* ActorDescItem = Item.CastTo<FDataLayerActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return ActorDescInstance->GetActorNameString();
			}
		}
		return FString();
	});

	FGetTextForItem InternalInitialRuntimeStateInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FDataLayerTreeItem* DataLayerItem = Item.CastTo<FDataLayerTreeItem>())
		{
			if (const UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer())
			{
				if (DataLayerInstance->IsRuntime())
				{
					return GetDataLayerRuntimeStateName(DataLayerInstance->GetInitialRuntimeState());
				}
			}
		}
		return FString();
	});

	FGetTextForItem InternalStreamingPriorityInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
			if (const FDataLayerTreeItem* DataLayerItem = Item.CastTo<FDataLayerTreeItem>())
			{
				if (const UDataLayerInstance* DataLayerInstance = DataLayerItem->GetDataLayer())
				{
					if (DataLayerInstance->GetStreamingPriority() != 0)
					{
						return FString::FromInt(DataLayerInstance->GetStreamingPriority());
					}
				}
			}
			return FString();
	});

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.OutlinerIdentifier = TEXT("DataLayerEditorOutliner");
	InitOptions.bShowHeaderRow = true;
	InitOptions.bShowParentTree = true;
	InitOptions.bShowCreateNewFolder = false;
	InitOptions.bShowTransient = true;
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this](SSceneOutliner* Outliner) { return new FDataLayerMode(FDataLayerModeParams(Outliner, this, nullptr)); });
	InitOptions.ColumnMap.Add(FDataLayerOutlinerIsVisibleColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FDataLayerOutlinerIsVisibleColumn(InSceneOutliner)); })));
	InitOptions.ColumnMap.Add(FDataLayerOutlinerIsLoadedInEditorColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 1, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FDataLayerOutlinerIsLoadedInEditorColumn(InSceneOutliner)); })));
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 2, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));
	InitOptions.ColumnMap.Add(FDataLayerOutlinerDeleteButtonColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FDataLayerOutlinerDeleteButtonColumn(InSceneOutliner)); })));
	InitOptions.ColumnMap.Add("ID Name", FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 20, FCreateSceneOutlinerColumn::CreateStatic(&FTextInfoColumn::CreateTextInfoColumn, FName("ID Name"), InternalNameInfoText, LOCTEXT("DataLayerIDNameToolTip", "ID Name")), true, TOptional<float>(), LOCTEXT("DataLayerIDName", "ID Name")));
	InitOptions.ColumnMap.Add("Initial State", FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 20, FCreateSceneOutlinerColumn::CreateStatic(&FTextInfoColumn::CreateTextInfoColumn, FName("Initial State"), InternalInitialRuntimeStateInfoText, LOCTEXT("DataLayerInitialRuntimeStateToolTip", "Initial Runtime State")), true, TOptional<float>(), LOCTEXT("DataLayerInitialState", "Initial State")));
	InitOptions.ColumnMap.Add(FDataLayerOutlinerHasErrorsColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 100, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FDataLayerOutlinerHasErrorsColumn(InSceneOutliner)); }), false));
	InitOptions.ColumnMap.Add(FDataLayerOutlinerDebugColorColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 100, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FDataLayerOutlinerDebugColorColumn(InSceneOutliner)); })));
	InitOptions.ColumnMap.Add("Streaming Priority", FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 20, FCreateSceneOutlinerColumn::CreateStatic(&FTextInfoColumn::CreateTextInfoColumn, FName("Streaming Priority"), InternalStreamingPriorityInfoText, LOCTEXT("DataLayerStreamingPriorityToolTip", "Streaming Priority")), true, TOptional<float>(), LOCTEXT("DataLayerStreamingPriority", "Streaming Priority")));
	DataLayerOutliner = SNew(SDataLayerOutliner, InitOptions).IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());

	SAssignNew(DataLayerContentsSection, SBorder)
	.Padding(5)
	.BorderImage(FAppStyle::GetBrush("NoBrush"))
	.Content()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			// Data Layer Outliner
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			+ SSplitter::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					DataLayerOutliner.ToSharedRef()
				]
			]
			// Details
			+ SSplitter::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(2, 4, 0, 0)
				[
					DetailsWidget.ToSharedRef()
				]
			]
		]
	];

	//////////////////////////////////////////////////////////////////////////
	//	DataLayer Browser
	ChildSlot
	[
		SAssignNew(ContentAreaBox, SVerticalBox)
		.IsEnabled_Lambda([]()
		{
			if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
			{
				if (ULevel* CurrentLevel = World->GetCurrentLevel() && World->GetCurrentLevel() != World->PersistentLevel ? World->GetCurrentLevel() : nullptr)
				{
					World = CurrentLevel->GetTypedOuter<UWorld>();
				}
				return UWorld::IsPartitionedWorld(World);
			}
			return false;
		})
	];

	InitializeDataLayerBrowser();
}

void SDataLayerBrowser::SyncDataLayerBrowserToDataLayer(const UDataLayerInstance* DataLayer)
{
	FSceneOutlinerTreeItemPtr Item = DataLayerOutliner->GetTreeItem(DataLayer);
	if (Item.IsValid())
	{
		DataLayerOutliner->SetItemSelection(Item, true, ESelectInfo::OnMouseClick);
		FSceneOutlinerTreeItemPtr Parent = Item->GetParent();
		while(Parent.IsValid())
		{
			DataLayerOutliner->SetItemExpansion(Parent, true);
			Parent = Parent->GetParent();
		};
	}
}

void SDataLayerBrowser::OnSelectionChanged(TSet<TWeakObjectPtr<const UDataLayerInstance>>& InSelectedDataLayersSet)
{
	SelectedDataLayersSet = InSelectedDataLayersSet;
	TArray<UObject*> SelectedDataLayers;
	for (const auto& WeakDataLayer : SelectedDataLayersSet)
	{
		if (WeakDataLayer.IsValid())
		{
			UDataLayerInstance* DataLayer = const_cast<UDataLayerInstance*>(WeakDataLayer.Get());
			SelectedDataLayers.Add(DataLayer);
		}
	}
	DetailsWidget->SetObjects(SelectedDataLayers, /*bForceRefresh*/ true);
}


void SDataLayerBrowser::InitializeDataLayerBrowser()
{
	ContentAreaBox->ClearChildren();
	ContentAreaBox->AddSlot()
	.AutoHeight()
	.FillHeight(1.0f)
	[
		DataLayerContentsSection.ToSharedRef()
	];

	ContentAreaBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Bottom)
	.MaxHeight(23)
	[
		DataLayerContentsHeader.ToSharedRef()
	];
}

#undef LOCTEXT_NAMESPACE