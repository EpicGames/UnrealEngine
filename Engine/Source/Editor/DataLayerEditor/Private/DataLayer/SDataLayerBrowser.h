// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SBorder;
class SButton;
class SMultiLineEditableTextBox;
class SVerticalBox;
class UDataLayerInstance;

//////////////////////////////////////////////////////////////////////////
// SDataLayerBrowser

class SDataLayerBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDataLayerBrowser ) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct(const FArguments& InArgs);

	void SyncDataLayerBrowserToDataLayer(const UDataLayerInstance* DataLayer);
	void OnSelectionChanged(TSet<TWeakObjectPtr<const UDataLayerInstance>>& SelectedDataLayersSet);

private:
	void InitializeDataLayerBrowser();

	TSet<TWeakObjectPtr<const UDataLayerInstance>> SelectedDataLayersSet;
	TSharedPtr<class SDataLayerOutliner> DataLayerOutliner;
	TSharedPtr<class IDetailsView> DetailsWidget;
	TSharedPtr<SButton> ToggleModeButton;
	TSharedPtr<SVerticalBox> ContentAreaBox;
	TSharedPtr<SBorder> DataLayerContentsSection;
	TSharedPtr<SBorder> DataLayerContentsHeader;
};
