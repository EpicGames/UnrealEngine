// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataLayerOutliner.h"

#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayer/DataLayerTreeItem.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

#define LOCTEXT_NAMESPACE "DataLayer"

void SDataLayerOutliner::CustomAddToToolbar(TSharedPtr<SHorizontalBox> Toolbar)
{
	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.IsEnabled(this, &SDataLayerOutliner::CanAddSelectedActorsToSelectedDataLayers)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("AddSelectedActorsToSelectedDataLayersTooltip", "Add selected actors to selected Data Layers"))
			.OnClicked(this, &SDataLayerOutliner::OnAddSelectedActorsToSelectedDataLayersClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("DataLayerBrowser.AddSelection"))
			]
		];

	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.IsEnabled(this, &SDataLayerOutliner::CanRemoveSelectedActorsFromSelectedDataLayers)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("RemoveSelectedActorsFromSelectedDataLayersTooltip", "Remove selected actors from selected Data Layers"))
			.OnClicked(this, &SDataLayerOutliner::OnRemoveSelectedActorsFromSelectedDataLayersClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("DataLayerBrowser.RemoveSelection"))
			]
		];
}

TArray<UDataLayerInstance*> SDataLayerOutliner::GetSelectedDataLayers() const
{
	FSceneOutlinerItemSelection ItemSelection(GetSelection());
	TArray<FDataLayerTreeItem*> SelectedDataLayerItems;
	ItemSelection.Get<FDataLayerTreeItem>(SelectedDataLayerItems);
	TArray<UDataLayerInstance*> ValidSelectedDataLayers;
	Algo::TransformIf(SelectedDataLayerItems, ValidSelectedDataLayers, [](const auto Item) { return Item && Item->GetDataLayer(); }, [](const auto Item) { return Item->GetDataLayer(); });
	return ValidSelectedDataLayers;
}

bool SDataLayerOutliner::CanAddSelectedActorsToSelectedDataLayers() const
{
	if (GEditor->GetSelectedActorCount() > 0)
	{
		TArray<UDataLayerInstance*> SelectedDataLayerInstances = GetSelectedDataLayers();
		if (!SelectedDataLayerInstances.IsEmpty())
		{
			TArray<AActor*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

			const bool bSelectedActorsAreAllUserManaged = Algo::AllOf(SelectedActors, [](const AActor* Actor) { return Actor->IsUserManaged(); });
			const bool bAllSelectedDataLayersCanAddActors = Algo::AllOf(SelectedDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->CanUserAddActors(); });

			return bAllSelectedDataLayersCanAddActors && bSelectedActorsAreAllUserManaged;
		}
	}
	return false;
}

bool SDataLayerOutliner::CanRemoveSelectedActorsFromSelectedDataLayers() const
{
	return CanAddSelectedActorsToSelectedDataLayers();
}

FReply SDataLayerOutliner::OnAddSelectedActorsToSelectedDataLayersClicked()
{
	if (CanAddSelectedActorsToSelectedDataLayers())
	{
		TArray<UDataLayerInstance*> SelectedDataLayers = GetSelectedDataLayers();
		const FScopedTransaction Transaction(LOCTEXT("AddSelectedActorsToSelectedDataLayers", "Add Selected Actor(s) to Selected Data Layer(s)"));
		UDataLayerEditorSubsystem::Get()->AddSelectedActorsToDataLayers(SelectedDataLayers);
	}
	return FReply::Handled();
}

FReply SDataLayerOutliner::OnRemoveSelectedActorsFromSelectedDataLayersClicked()
{
	if (CanRemoveSelectedActorsFromSelectedDataLayers())
	{
		TArray<UDataLayerInstance*> SelectedDataLayers = GetSelectedDataLayers();
		const FScopedTransaction Transaction(LOCTEXT("RemoveSelectedActorsFromSelectedDataLayers", "Remove Selected Actors from Selected Data Layers"));
		UDataLayerEditorSubsystem::Get()->RemoveSelectedActorsFromDataLayers(SelectedDataLayers);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE