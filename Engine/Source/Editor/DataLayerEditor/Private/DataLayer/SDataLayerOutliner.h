// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Input/Reply.h"
#include "SSceneOutliner.h"
#include "Templates/SharedPointer.h"

struct FSceneOutlinerInitializationOptions;

class SDataLayerOutliner : public SSceneOutliner
{
public:
	void Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions)
	{
		SSceneOutliner::Construct(InArgs, InitOptions);
	}

	SDataLayerOutliner() {}
	virtual ~SDataLayerOutliner() = default;

	void CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar) override;
private:
	bool CanAddSelectedActorsToSelectedDataLayers() const;
	bool CanRemoveSelectedActorsFromSelectedDataLayers() const;
	FReply OnAddSelectedActorsToSelectedDataLayersClicked();
	FReply OnRemoveSelectedActorsFromSelectedDataLayersClicked();
	TArray<class UDataLayerInstance*> GetSelectedDataLayers() const;
};
