// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldDataLayersTreeItem.h"

#include "Containers/EnumAsByte.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "ISceneOutliner.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SceneOutlinerPublicTypes.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateIconFinder.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldTreeItem.h"

class SWidget;
template <typename ItemType> class STableRow;

#define LOCTEXT_NAMESPACE "DataLayer"

const FSceneOutlinerTreeItemType FWorldDataLayersTreeItem::Type(&ISceneOutlinerTreeItem::Type);

struct SWorldDataLayersTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWorldDataLayersTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FWorldDataLayersTreeItem& WorldDataLayersItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FWorldDataLayersTreeItem>(WorldDataLayersItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
				.HeightOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
				[
					SNew(SImage)
					.Image(FSlateIconFinder::FindIconBrushForClass(UWorld::StaticClass()))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("LevelIcon_Tooltip", "Level"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SWorldDataLayersTreeLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SWorldDataLayersTreeLabel::GetForegroundColor)
			]
		];
	}

private:
	TWeakPtr<FWorldDataLayersTreeItem> TreeItemPtr;

	bool IsInActorEditorContext() const
	{
		auto Item = TreeItemPtr.Pin();
		const AWorldDataLayers* WorldDataLayers = Item.IsValid() ? Item->GetWorldDataLayers() : nullptr;
		const UWorld* OwningWorld = WorldDataLayers ? WorldDataLayers->GetWorld() : nullptr;
		ULevel* CurrentLevel = (OwningWorld && OwningWorld->GetCurrentLevel() && !OwningWorld->GetCurrentLevel()->IsPersistentLevel()) ? OwningWorld->GetCurrentLevel() : nullptr;
		return CurrentLevel && (CurrentLevel == WorldDataLayers->GetLevel());
	}

	FText GetDisplayText() const
	{
		auto Item = TreeItemPtr.Pin();
		if (Item.IsValid())
		{
			FText SuffixText = FText::GetEmpty();
			if (IsInActorEditorContext())
			{
				SuffixText = FText(LOCTEXT("IsCurrentSuffix", " (Current)"));
			}
			return FText::Format(LOCTEXT("WorldDataLayersDisplayText", "{0}{1}"), FText::FromString(Item->GetDisplayString()), SuffixText);
		}
		return FText();
	}

	FSlateColor GetForegroundColor() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}
		else if (IsInActorEditorContext())
		{
			return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
		}

		return FSlateColor::UseForeground();
	}
};

FWorldDataLayersTreeItem::FWorldDataLayersTreeItem(AWorldDataLayers* InWorldDataLayers)
	: ISceneOutlinerTreeItem(Type)
	, WorldDataLayers(InWorldDataLayers)
	, ID(InWorldDataLayers)
{
	Flags.bIsExpanded = true;
}

FString FWorldDataLayersTreeItem::GetDisplayString() const
{
	if (const AWorldDataLayers* WorldDataLayersPtr = WorldDataLayers.Get())
	{
		check(WorldDataLayersPtr->GetLevel());
		return SceneOutliner::GetWorldDescription(WorldDataLayersPtr->GetLevel()->GetTypedOuter<UWorld>()).ToString();
	}
	return LOCTEXT("UnknownWorldDataLayers", "Unknown").ToString();
}

bool FWorldDataLayersTreeItem::CanInteract() const
{
	return !IsReadOnly();
}

bool FWorldDataLayersTreeItem::IsReadOnly() const
{
	const AWorldDataLayers* WorldDataLayersPtr = WorldDataLayers.Get();
	UWorld* WorldPtr = WorldDataLayersPtr ? WorldDataLayersPtr->GetLevel()->GetTypedOuter<UWorld>() : nullptr;
	return (WorldDataLayersPtr && WorldDataLayersPtr->IsReadOnly()) || !WorldPtr || (WorldPtr->WorldType != EWorldType::Editor);
}

TSharedRef<SWidget> FWorldDataLayersTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SWorldDataLayersTreeLabel, *this, Outliner, InRow);
}

int32 FWorldDataLayersTreeItem::GetSortPriority() const
{
	return CanInteract() ? 0 : 1;
}

#undef LOCTEXT_NAMESPACE