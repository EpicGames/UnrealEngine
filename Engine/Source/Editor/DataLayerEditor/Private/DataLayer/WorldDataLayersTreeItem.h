// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ISceneOutlinerTreeItem.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"

class ISceneOutliner;
class SWidget;
template <typename ItemType> class STableRow;

struct FWorldDataLayersTreeItem : ISceneOutlinerTreeItem
{
public:
	FWorldDataLayersTreeItem(AWorldDataLayers* InWorldDataLayers);
	AWorldDataLayers* GetWorldDataLayers() const { return WorldDataLayers.Get(); }

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return WorldDataLayers.IsValid(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return ID; }
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual bool ShouldShowVisibilityState() const override { return false; }
	/* End ISceneOutlinerTreeItem Implementation */

	int32 GetSortPriority() const;

	static const FSceneOutlinerTreeItemType Type;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const AWorldDataLayers*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const AWorldDataLayers*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(GetWorldDataLayers());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(GetWorldDataLayers());
	}

private:
	bool IsReadOnly() const;
	TWeakObjectPtr<AWorldDataLayers> WorldDataLayers;
	const FObjectKey ID;
};