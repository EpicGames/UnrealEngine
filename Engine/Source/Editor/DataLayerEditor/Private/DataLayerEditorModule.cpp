// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLayerEditorModule.h"

#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayer/DataLayerInstanceCustomization.h"
#include "DataLayer/DataLayerNameEditSink.h"
#include "DataLayer/DataLayerPropertyTypeCustomization.h"
#include "DataLayer/DataLayerPropertyTypeCustomizationHelper.h"
#include "DataLayer/SDataLayerBrowser.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "EditorWidgetsModule.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "ObjectNameEditSinkRegistry.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "LevelEditorMenuContext.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Selection.h"
#include "Algo/AnyOf.h"
#include "Editor.h"

class AActor;

IMPLEMENT_MODULE(FDataLayerEditorModule, DataLayerEditor );

#define LOCTEXT_NAMESPACE "DataLayerEditorModule"

static const FName NAME_DataLayerInstance(TEXT("DataLayerInstance"));

void FDataLayerEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(NAME_DataLayerInstance, FOnGetDetailCustomizationInstance::CreateStatic(&FDataLayerInstanceDetails::MakeInstance));

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	EditorWidgetsModule.GetObjectNameEditSinkRegistry()->RegisterObjectNameEditSink(MakeShared<FDataLayerNameEditSink>());

	UToolMenus::Get()->RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FDataLayerEditorModule::RegisterMenus));
}

void FDataLayerEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout(NAME_DataLayerInstance);
	}
}

TSharedRef<SWidget> FDataLayerEditorModule::CreateDataLayerBrowser()
{
	TSharedRef<SWidget> NewDataLayerBrowser = SNew(SDataLayerBrowser);
	DataLayerBrowser = NewDataLayerBrowser;
	return NewDataLayerBrowser;
}

void FDataLayerEditorModule::SyncDataLayerBrowserToDataLayer(const UDataLayerInstance* DataLayerInstance)
{
	if (DataLayerBrowser.IsValid())
	{
		TSharedRef<SDataLayerBrowser> Browser = StaticCastSharedRef<SDataLayerBrowser>(DataLayerBrowser.Pin().ToSharedRef());
		Browser->SyncDataLayerBrowserToDataLayer(DataLayerInstance);
	}
}

bool FDataLayerEditorModule::AddActorToDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers)
{
	return UDataLayerEditorSubsystem::Get()->AddActorToDataLayers(Actor, DataLayers);
}

void FDataLayerEditorModule::SetActorEditorContextCurrentExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	UDataLayerEditorSubsystem::Get()->SetActorEditorContextCurrentExternalDataLayer(InExternalDataLayerAsset);
}

bool FDataLayerEditorModule::MoveActorsToExternalDataLayer(const TArray<AActor*>& InSelectedActors, const UExternalDataLayerInstance* InExternalDataLayerInstance, FText* OutReason)
{
	if (InSelectedActors.IsEmpty())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("NoActorToProcess", "No actor to process");
		}
		return false;
	}
	return FExternalDataLayerHelper::MoveActorsToExternalDataLayer(InSelectedActors, InExternalDataLayerInstance, OutReason);
}

namespace UE::Private::DataLayerEditorModule
{
	void LogWarningAndNotify(const FText& InWarningMessage, const FText& InDetailedMessage)
	{
		UE_LOGF(LogWorldPartition, Warning, "%ls : %ls", *InWarningMessage.ToString(), *InDetailedMessage.ToString());
		FNotificationInfo WarningInfo(InWarningMessage);
		WarningInfo.SubText = InDetailedMessage;
		WarningInfo.ExpireDuration = 5.0f;
		WarningInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
		WarningInfo.HyperlinkText = LOCTEXT("ShowMessageLogHyperlink", "Show Output Log");
		FSlateNotificationManager::Get().AddNotification(WarningInfo);
	}
}

void FDataLayerEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);
	FToolUIAction RemoveExternalDataLayerAction;
	RemoveExternalDataLayerAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		FText FailureReason;
		FScopedTransaction Transaction(LOCTEXT("RemoveActorsFromExternalDataLayer", "Remove Actor(s) From External Data Layer"));
		TArray<AActor*> ActorsWithExternalDataLayer;
		Algo::TransformIf(GetSelectedActors(), ActorsWithExternalDataLayer, [](const AActor* Actor) { return Actor && Actor->GetExternalDataLayerAsset(); }, [](AActor* Actor) { return Actor; });
		if (!MoveActorsToExternalDataLayer(ActorsWithExternalDataLayer, nullptr, &FailureReason))
		{
			Transaction.Cancel();
			const FText WarningMessage = LOCTEXT("RemoveActorsFromExternalDataLayerFailed", "Failed to remove actor(s) from External Data Layer");
			UE::Private::DataLayerEditorModule::LogWarningAndNotify(WarningMessage, FailureReason);
		}
	});
	RemoveExternalDataLayerAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		return Algo::AnyOf(GetSelectedActors(), [](AActor* Actor) { return Actor && Actor->GetExternalDataLayerAsset(); });
	});

	auto MoveToExternalDataLayerMenu = [this](UToolMenu* Menu)
	{
		TSet<const UExternalDataLayerAsset*> ExternalDataLayerAssets;
		Algo::TransformIf(GetSelectedActors(), ExternalDataLayerAssets, [](AActor* Actor) { return Actor && Actor->GetExternalDataLayerAsset(); }, [](AActor* Actor) { return Actor->GetExternalDataLayerAsset(); });

		FToolMenuSection& Section = Menu->AddSection("External Data Layer Picker");
		TSharedRef< SWidget > MenuWidget =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.MaxHeight(400.0f)
			[
				FDataLayerPropertyTypeCustomizationHelper::CreateDataLayerMenu(
					// OnDataLayerSelectedFunction
					[this](const UDataLayerInstance* DataLayerInstance)
					{
						const UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(DataLayerInstance);
						if (ExternalDataLayerInstance && ExternalDataLayerInstance->GetExternalDataLayerAsset())
						{
							FText FailureReason;
							FScopedTransaction Transaction(LOCTEXT("MoveActorsToExternalDataLayer", "Move Actor(s) To External Data Layer"));
							if (!MoveActorsToExternalDataLayer(GetSelectedActors(), ExternalDataLayerInstance, &FailureReason))
							{
								Transaction.Cancel();
								const FText WarningMessage = LOCTEXT("MoveActorsToExternalDataLayerFailed", "Failed to move actor(s) to External Data Layer");
								UE::Private::DataLayerEditorModule::LogWarningAndNotify(WarningMessage, FailureReason);
							}
						}
					},
					// OnShouldFilterDataLayerInstanceFunction
					[ExternalDataLayerAssets](const UDataLayerInstance* DataLayerInstance)
					{
						const UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(DataLayerInstance);
						const UExternalDataLayerAsset* ExternalDataLayerAsset = ExternalDataLayerInstance ? ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
						return (!ExternalDataLayerAsset || ExternalDataLayerAssets.Contains(ExternalDataLayerAsset));
					})
			];

		Section.AddEntry(FToolMenuEntry::InitWidget("PickExternalDataLayer", MenuWidget, FText::GetEmpty(), false));
	};

	auto FillExternalDataLayerMenu = [this, RemoveExternalDataLayerAction, MoveToExternalDataLayerMenu](UToolMenu* SubMenu)
	{
		FToolMenuSection& ExternalDataLayerSection = SubMenu->AddSection("External Data Layer");
		ExternalDataLayerSection.AddMenuEntry(
			"ActorRemoveFromExternalDataLayerMenu",
			LOCTEXT("RemoveActorsFromTheirExternalDataLayerMenuEntry", "Remove Actors(s)"),
			LOCTEXT("RemoveActorsFromTheirExternalDataLayerMenu_ToolTip", "Remove Actor(s) from their External Data Layer"),
			FSlateIcon(),
			RemoveExternalDataLayerAction);

		ExternalDataLayerSection.AddSubMenu(
			"MoveToExternalDataLayerSubMenu",
			LOCTEXT("MoveActorsToPickedExternalDataLayerSubMenu", "Move Actor(s) To"),
			LOCTEXT("MoveActorsToPickedExternalDataLayerSubMenu_ToolTip", "Move Actor(s) to picked External Data Layer"),
			FNewToolMenuDelegate::CreateLambda(MoveToExternalDataLayerMenu));
	};

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu");
	FToolMenuSection& ActorSection = Menu->FindOrAddSection("ActorOptions");

	FToolUIAction FillExternalDataLayerMenuAction;
	FillExternalDataLayerMenuAction.IsActionVisibleDelegate = FToolMenuCanExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
		return EditorWorld && EditorWorld->IsPartitionedWorld();
	});
	FillExternalDataLayerMenuAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		TArray<AActor*> SelectedActors = GetSelectedActors();
		if (Algo::AnyOf(GetSelectedActors(), [](AActor* Actor) { return Actor && !Actor->IsUserManaged(); }))
		{
			return false;
		}
		UExternalDataLayerManager* ExternalDataLayerManager = SelectedActors.Num() ? UExternalDataLayerManager::GetExternalDataLayerManager(SelectedActors[0]) : nullptr;
		return ExternalDataLayerManager && ExternalDataLayerManager->HasInjectedExternalDataLayerAssets();
	});

	FToolUIAction ApplyActorEditorContextDataLayersToActorAction;
	ApplyActorEditorContextDataLayersToActorAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		FScopedTransaction Transaction(LOCTEXT("ApplyActorEditorContextDataLayersToActors", "Apply Actor Editor Context Data Layers To Actor(s)"));
		if (!UDataLayerEditorSubsystem::Get()->ApplyActorEditorContextDataLayersToActors(GetSelectedActors()))
		{
			const FText WarningMessage = LOCTEXT("ApplyActorEditorContextDataLayersToActorFailed", "Failed apply actor editor context data layers to actor(s)");
			const FText DetailedMessage = LOCTEXT("ApplyActorEditorContextDataLayersToActorFailedDetailed", "See log for details.");
			UE::Private::DataLayerEditorModule::LogWarningAndNotify(WarningMessage, DetailedMessage);
		}
	});

	auto FillDataLayerMenu = [this, ApplyActorEditorContextDataLayersToActorAction, FillExternalDataLayerMenu, FillExternalDataLayerMenuAction](UToolMenu* SubMenu)
	{
		FToolMenuSection& DataLayerSection = SubMenu->AddSection("Data Layer");
		DataLayerSection.AddMenuEntry(
			"ApplyActorEditorContextDataLayersToActorsMenu",
			LOCTEXT("ApplyActorEditorContextDataLayersToActorMenuEntry", "Apply Actor Editor Context"),
			LOCTEXT("ApplyActorEditorContextDataLayersToActorMenu_ToolTip", "Applies the Actor Editor Context's Current Data Layers to the actor(s)"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataLayer.Editor"),
			ApplyActorEditorContextDataLayersToActorAction);

		DataLayerSection.AddSubMenu(
			"ExternalDataLayerSubMenu",
			LOCTEXT("ExternalDataLayerSubMenu", "External Data Layer"),
			LOCTEXT("ExternalDataLayerSubMenu_ToolTip", "External Data Layer Utils"),
			FNewToolMenuDelegate::CreateLambda(FillExternalDataLayerMenu),
			FillExternalDataLayerMenuAction,
			EUserInterfaceActionType::Button,
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataLayer.External"));
	};

	FToolUIAction FillDataLayerMenuAction;
	FillExternalDataLayerMenuAction.IsActionVisibleDelegate = FToolMenuCanExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
		return EditorWorld && EditorWorld->IsPartitionedWorld();
	});

	ActorSection.AddSubMenu(
		"ExternalDataLayerSubMenu",
		LOCTEXT("DataLayerSubMenu", "Data Layer"),
		LOCTEXT("DataLayerSubMenu_ToolTip", "Data Layer Utils"),
		FNewToolMenuDelegate::CreateLambda(FillDataLayerMenu),
		FillDataLayerMenuAction,
		EUserInterfaceActionType::Button,
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataLayer.Editor"));
}

TArray<AActor*> FDataLayerEditorModule::GetSelectedActors() const
{
	TArray<UObject*> SelectedObjects;
	GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), /*out*/ SelectedObjects);
	TArray<AActor*> SelectedActors;
	Algo::TransformIf(SelectedObjects, SelectedActors, [](UObject* Object) { return Object && Object->IsA<AActor>(); }, [](UObject* Object) { return Cast<AActor>(Object); });
	return SelectedActors;
}

#undef LOCTEXT_NAMESPACE