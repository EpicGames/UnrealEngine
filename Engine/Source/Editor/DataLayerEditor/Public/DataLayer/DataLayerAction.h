// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

enum class EDataLayerAction : uint8
{
	Add,
	Modify,
	Delete,
	Rename,
	Reset
};