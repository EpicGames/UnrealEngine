// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class AActor;

typedef TPair< TWeakObjectPtr<AActor>, TWeakObjectPtr<UDataLayerInstance>> FDataLayerActorMoveElement;

/** Drag/drop operation for dragging layers in the editor */
class FDataLayerDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDataLayerDragDropOp, FDecoratedDragDropOp)

	void Init(const TArray<TWeakObjectPtr<UDataLayerInstance>>& InDataLayerInstances)
	{
		for (const auto& DataLayerInstance : InDataLayerInstances)
		{
			if (DataLayerInstance.IsValid())
			{
				DataLayerInstances.Emplace(DataLayerInstance.Get());
			}
		}
	}

	/** The labels of the layers being dragged */
	TArray< TWeakObjectPtr<UDataLayerInstance> > DataLayerInstances;

	virtual void Construct() override;
};

/** Drag/drop operation for moving actor(s) in a data layer in the editor */
class FDataLayerActorMoveOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDataLayerActorMoveOp, FDecoratedDragDropOp)

	/** Actor that we are dragging */
	TArray<FDataLayerActorMoveElement> DataLayerActorMoveElements;

	virtual void Construct() override;
};