// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/WorldEditorState.h"
#include "DataLayerEditorState.generated.h"

class UDataLayerAsset;

UCLASS()
class UDataLayerEditorState : public UWorldDependantEditorState
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEditorState Interface
	virtual FText GetCategoryText() const override;
private:
	virtual FOperationResult CaptureState() override;
	virtual FOperationResult RestoreState() const override;
	//~ End UEditorState Interface

private:
	UPROPERTY(EditAnywhere, Category = DataLayers)
	TArray<TObjectPtr<const UDataLayerAsset>> NotLoadedDataLayers;

	UPROPERTY(EditAnywhere, Category = DataLayers)
	TArray<TObjectPtr<const UDataLayerAsset>> LoadedDataLayers;
};
