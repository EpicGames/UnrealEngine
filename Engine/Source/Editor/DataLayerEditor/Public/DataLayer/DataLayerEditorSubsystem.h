// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorEditorContextState.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "DataLayerAction.h"
#include "Delegates/Delegate.h"
#include "EditorSubsystem.h"
#include "Engine/World.h"
#include "IActorEditorContextClient.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Tickable.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "LevelEditorDragDropHandler.h"

#include "DataLayerEditorSubsystem.generated.h"

#define UE_API DATALAYEREDITOR_API

class AActor;
class AWorldDataLayers;
class FDataLayersBroadcast;
class FLevelEditorViewportClient;
class FSubsystemCollectionBase;
class SWidget;
class UDataLayerAsset;
class UEditorEngine;
class ULevel;
class UObject;
class UWorld;
class UWorldPartition;
class UExternalDataLayerAsset;
struct FFrame;
enum class EExternalDataLayerRegistrationState : uint8;
template<typename TItemType> class IFilter;
namespace WorldPartitionTests { class FExternalDataLayerTest; }

USTRUCT(BlueprintType)
struct FDataLayerCreationParameters
{
	GENERATED_USTRUCT_BODY()

	UE_API FDataLayerCreationParameters();

	// Required if bIsPrivate is false. Will assign the asset to the created instance.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Data Layer")
	TObjectPtr<UDataLayerAsset> DataLayerAsset;

	// Optional. Will default at the level WorldDataLayers if unset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Data Layer")
	TObjectPtr<AWorldDataLayers> WorldDataLayers;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Data Layer")
	bool bIsPrivate; 
};

UCLASS()
class UActorEditorContextDataLayerState : public UActorEditorContextClientState
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category="Data Layers")
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;
	
	UPROPERTY(VisibleAnywhere, Category="Data Layers")
	TArray<TSoftObjectPtr<const UDataLayerAsset>> DataLayerAssets;
};

UCLASS(BlueprintType, MinimalAPI)
class UDataLayerEditorSubsystem final : public UEditorSubsystem, public IActorEditorContextClient, public FTickableGameObject
{
	GENERATED_BODY()

public:
	UE_API UDataLayerEditorSubsystem();

	static UE_API UDataLayerEditorSubsystem* Get();

	typedef IFilter<const TWeakObjectPtr<AActor>&> FActorFilter;

	/**
	 *	Prepares for use
	 */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	/**
	 *	Internal cleanup
	 */
	UE_API virtual void Deinitialize() override;

	/**
	 *	Destructor
	 */
	virtual ~UDataLayerEditorSubsystem() = default;

	//~ Begin FTickableGameObject interface
	UE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	virtual bool IsTickableInEditor() const { return true; }
	UE_API virtual ETickableTickType GetTickableTickType() const override;
	UE_API virtual bool IsTickable() const override;
	UE_API virtual void Tick(float DeltaTime) override;
	TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UDataLayerEditorSubsystem, STATGROUP_Tickables); }
	//~ End FTickableGameObject interface

	//~ Begin UObject overrides.
	UE_API virtual void BeginDestroy() override;
	// ~End UObject overrides.

	//~ Begin IActorEditorContextClient interface
	UE_API virtual void OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor = nullptr) override;
	UE_API virtual void CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const override;
	UE_API virtual void RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection) override;
	UE_API virtual bool GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const override;
	virtual bool CanResetContext(UWorld* InWorld) const override { return true; };
	UE_API virtual TSharedRef<SWidget> GetActorEditorContextWidget(UWorld* InWorld) const override;
	virtual FOnActorEditorContextClientChanged& GetOnActorEditorContextClientChanged() override { return ActorEditorContextClientChanged; }
	//~ End IActorEditorContextClient interface
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void AddToActorEditorContext(UDataLayerInstance* InDataLayerInstance);
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void RemoveFromActorEditorContext(UDataLayerInstance* InDataLayerInstance);
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SetActorEditorContextCurrentExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API const UExternalDataLayerAsset* GetActorEditorContextCurrentExternalDataLayer() const;

	UE_API TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<const UDataLayerAsset*> DataLayerAssets) const;

	template<class DataLayerType, typename ...Args>
	DataLayerType* CreateDataLayerInstance(AWorldDataLayers* WorldDataLayers, Args&&... InArgs);

	/* Broadcasts whenever one or more DataLayers are modified
	 *
	 * Actions
	 * Add    : The specified ChangedDataLayer is a newly created UDataLayerInstance
	 * Modify : The specified ChangedDataLayer was just modified, if ChangedDataLayer is invalid then multiple DataLayers were modified.
	 *          ChangedProperty specifies what field on the UDataLayerInstance was changed, if NAME_None then multiple fields were changed
	 * Delete : A DataLayer was deleted
	 * Rename : The specified ChangedDataLayer was just renamed
	 * Reset  : A large amount of changes have occurred to a number of DataLayers.
	 */
	DECLARE_EVENT_ThreeParams(UDataLayerEditorSubsystem, FOnDataLayerChanged, const EDataLayerAction /*Action*/, const TWeakObjectPtr<const UDataLayerInstance>& /*ChangedDataLayer*/, const FName& /*ChangedProperty*/);
	FOnDataLayerChanged& OnDataLayerChanged() { return DataLayerChanged; }

	/** Broadcasts whenever one or more Actors changed UDataLayerInstances*/
	DECLARE_EVENT_OneParam(UDataLayerEditorSubsystem, FOnActorDataLayersChanged, const TWeakObjectPtr<AActor>& /*ChangedActor*/);
	FOnActorDataLayersChanged& OnActorDataLayersChanged() { return ActorDataLayersChanged; }

	/** Broadcasts whenever one or more DataLayers editor loading state changed */
	DECLARE_EVENT_OneParam(UDataLayerEditorSubsystem, FOnActorDataLayersEditorLoadingStateChanged, bool /*bIsFromUserChange*/);
	FOnActorDataLayersEditorLoadingStateChanged& OnActorDataLayersEditorLoadingStateChanged() { return DataLayerEditorLoadingStateChanged; }

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Operations on an individual actor.

	UE_DEPRECATED(all, "Use IsActorValidForDataLayerInstances instead.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use IsActorValidForDataLayerInstances instead."))
	bool IsActorValidForDataLayer(AActor* Actor) { return false; }

	/**
	 *	Checks to see if the specified actor is in an appropriate state to interact with DataLayers
	 *
	 *	@param	Actor				The actor to validate
	 *  @param  DataLayerInstances	The data layers used to do the validation
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool IsActorValidForDataLayerInstances(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayerInstances);

	/**
	 * Adds the actor to the DataLayer.
	 *
	 * @param	Actor		The actor to add to the DataLayer
	 * @param	DataLayer	The DataLayer to add the actor to
	 * @return				true if the actor was added.  false is returned if the actor already belongs to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddActorToDataLayer(AActor* Actor, UDataLayerInstance* DataLayer);

	/**
	 * Adds the provided actor to the DataLayers.
	 *
	 * @param	Actor		The actor to add to the provided DataLayers
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if the actor was added to at least one of the provided DataLayers.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddActorToDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers);

	/**
	 * Removes an actor from the specified DataLayer.
	 *
	 * @param	Actor			The actor to remove from the provided DataLayer
	 * @param	DataLayerToRemove	The DataLayer to remove the actor from
	 * @return					true if the actor was removed from the DataLayer.  false is returned if the actor already belonged to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorFromDataLayer(AActor* Actor, UDataLayerInstance* DataLayerToRemove);

	/**
	 * Removes the provided actor from the DataLayers.
	 *
	 * @param	Actor		The actor to remove from the provided DataLayers
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if the actor was removed from at least one of the provided DataLayers.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorFromDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers);

	/**
	 * Removes an actor from all DataLayers.
	 *
	 * @param	Actor			The actor to modify
	 * @return					true if the actor was changed.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorFromAllDataLayers(AActor* Actor);
	
	/**
	 * Removes an actor from all DataLayers.
	 *
	 * @param	Actor			The actors to modify
	 * @return					true if any actor was changed.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorsFromAllDataLayers(const TArray<AActor*>& Actors);

	/////////////////////////////////////////////////
	// Operations on multiple actors.

	/**
	 * Add the actors to the DataLayer
	 *
	 * @param	Actors		The actors to add to the DataLayer
	 * @param	DataLayer	The DataLayer to add to
	 * @return				true if at least one actor was added to the DataLayer.  false is returned if all the actors already belonged to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddActorsToDataLayer(const TArray<AActor*>& Actors, UDataLayerInstance* DataLayer);

	/**
	 * Add the actors to the DataLayers
	 *
	 * @param	Actors		The actors to add to the DataLayers
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if at least one actor was added to at least one DataLayer.  false is returned if all the actors already belonged to all specified DataLayers.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddActorsToDataLayers(const TArray<AActor*>& Actors, const TArray<UDataLayerInstance*>& DataLayers);

	/**
	 * Removes the actors from the specified DataLayer.
	 *
	 * @param	Actors			The actors to remove from the provided DataLayer
	 * @param	DataLayerToRemove	The DataLayer to remove the actors from
	 * @return					true if at least one actor was removed from the DataLayer.  false is returned if all the actors already belonged to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorsFromDataLayer(const TArray<AActor*>& Actors, UDataLayerInstance* DataLayer);

	/**
	 * Remove the actors from the DataLayers
	 *
	 * @param	Actors		The actors to remove to the DataLayers
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if at least one actor was removed from at least one DataLayer.  false is returned if none of the actors belonged to any of the specified DataLayers.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveActorsFromDataLayers(const TArray<AActor*>& Actors, const TArray<UDataLayerInstance*>& DataLayers);

	/////////////////////////////////////////////////
	// Operations on selected actors.

	/**
	 * Adds selected actors to the DataLayer.
	 *
	 * @param	DataLayer	A DataLayer.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddSelectedActorsToDataLayer(UDataLayerInstance* DataLayer);

	/**
	 * Adds selected actors to the DataLayers.
	 *
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the DataLayers.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool AddSelectedActorsToDataLayers(const TArray<UDataLayerInstance*>& DataLayers);

	/**
	 * Removes the selected actors from the DataLayer.
	 *
	 * @param	DataLayer	A DataLayer.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveSelectedActorsFromDataLayer(UDataLayerInstance* DataLayer);

	/**
	 * Removes selected actors from the DataLayers.
	 *
	 * @param	DataLayers	A valid list of DataLayers.
	 * @return				true if at least one actor was removed.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool RemoveSelectedActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers);


	/////////////////////////////////////////////////
	// Operations on actors in DataLayers
	
	/**
	 * Selects/de-selects actors belonging to the DataLayer.
	 *
	 * @param	DataLayer						A valid DataLayer.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified.
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SelectActorsInDataLayer(UDataLayerInstance* DataLayer, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden = false);

	/**
	 * Selects/de-selects actors belonging to the DataLayer.
	 *
	 * @param	DataLayer						A valid DataLayer.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified.
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @param	Filter	[optional]				Actor that don't pass the specified filter restrictions won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UE_API bool SelectActorsInDataLayer(UDataLayerInstance* DataLayer, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr<FActorFilter>& Filter);

	/**
	 * Selects/de-selects actors belonging to the DataLayers.
	 *
	 * @param	DataLayers						A valid list of DataLayers.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SelectActorsInDataLayers(const TArray<UDataLayerInstance*>& DataLayers, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden = false);

	/**
	 * Selects/de-selects actors belonging to the DataLayers.
	 *
	 * @param	DataLayers						A valid list of DataLayers.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @param	Filter	[optional]				Actor that don't pass the specified filter restrictions won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UE_API bool SelectActorsInDataLayers(const TArray<UDataLayerInstance*>& DataLayers, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr<FActorFilter>& Filter);

	/** 
	 * Pin/unpin actors belonging to the DataLayers.
	 * 
	 * @param	DataLayerInstances				A valid list of Data Layer Instances.
	 * @param	bPinned							If true actors are pinned; if false, actors are unpinned.
	 * 
	 */
	UE_API void SetActorsPinStateInDataLayers(const TArray<UDataLayerInstance*>& DataLayerInstances, const bool bPinned);


	/////////////////////////////////////////////////
	// Operations on actor viewport visibility regarding DataLayers



	/**
	 * Updates the provided actors visibility in the viewports
	 *
	 * @param	Actor						Actor to update
	 * @param	bOutSelectionChanged [OUT]	Whether the Editors selection changed
	 * @param	bOutActorModified [OUT]		Whether the actor was modified
	 * @param	bNotifySelectionChange		If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bRedrawViewports			If true the viewports will be redrawn; if false, they will not
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool UpdateActorVisibility(AActor* Actor, bool& bOutSelectionChanged, bool& bOutActorModified, const bool bNotifySelectionChange, const bool bRedrawViewports);

	/**
	 * Updates the visibility of all actors in the viewports
	 *
	 * @param	bNotifySelectionChange		If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bRedrawViewports			If true the viewports will be redrawn; if false, they will not
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports);


	/////////////////////////////////////////////////
	// Operations on DataLayers

	/**
	 *	Appends all the actors associated with the specified DataLayer.
	 *
	 *	@param	DataLayer			The DataLayer to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void AppendActorsFromDataLayer(UDataLayerInstance* DataLayer, TArray<AActor*>& InOutActors) const;

	/**
	 *	Appends all the actors associated with the specified DataLayer.
	 *
	 *	@param	DataLayer			The DataLayer to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UE_API void AppendActorsFromDataLayer(UDataLayerInstance* DataLayer, TArray<AActor*>& InOutActors, const TSharedPtr<FActorFilter>& Filter) const;
	
	/**
	 *	Appends all the actors associated with ANY of the specified DataLayers.
	 *
	 *	@param	DataLayers			The DataLayers to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void AppendActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers, TArray<AActor*>& InOutActors) const;

	/**
	 *	Appends all the actors associated with ANY of the specified DataLayers.
	 *
	 *	@param	DataLayers			The DataLayers to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UE_API void AppendActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers, TArray<AActor*>& InOutActors, const TSharedPtr<FActorFilter>& Filter) const;

	/**
	 *	Gets all the actors associated with the specified DataLayer. Analog to AppendActorsFromDataLayer but it returns rather than appends the actors.
	 *
	 *	@param	DataLayer			The DataLayer to find actors for.
	 *	@return						The list to assign the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API TArray<AActor*> GetActorsFromDataLayer(UDataLayerInstance* DataLayer) const;

	/**
	 *	Gets all the actors associated with the specified DataLayer. Analog to AppendActorsFromDataLayer but it returns rather than appends the actors.
	 *
	 *	@param	DataLayer			The DataLayer to find actors for.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 *	@return						The list to assign the found actors to.
	 */
	UE_API TArray<AActor*> GetActorsFromDataLayer(UDataLayerInstance* DataLayer, const TSharedPtr<FActorFilter>& Filter) const;

	/**
	 *	Gets all the actors associated with ANY of the specified DataLayers. Analog to AppendActorsFromDataLayers but it returns rather than appends the actors.
	 *
	 *	@param	DataLayers			The DataLayers to find actors for.
	 *	@return						The list to assign the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API TArray<AActor*> GetActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers) const;

	/**
	 *	Gets all the actors associated with ANY of the specified DataLayers. Analog to AppendActorsFromDataLayers but it returns rather than appends the actors.
	 *
	 *	@param	DataLayers			The DataLayers to find actors for.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 *	@return						The list to assign the found actors to.
	 */
	UE_API TArray<AActor*> GetActorsFromDataLayers(const TArray<UDataLayerInstance*>& DataLayers, const TSharedPtr<FActorFilter>& Filter) const;

	/**
	 * Changes the DataLayer's visibility to the provided state
	 *
	 * @param	DataLayer	The DataLayer to affect.
	 * @param	bIsVisible	If true the DataLayer will be visible; if false, the DataLayer will not be visible.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void SetDataLayerVisibility(UDataLayerInstance* DataLayer, const bool bIsVisible);

	/**
	 * Changes visibility of the DataLayers to the provided state
	 *
	 * @param	DataLayers	The DataLayers to affect
	 * @param	bIsVisible	If true the DataLayers will be visible; if false, the DataLayers will not be visible
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void SetDataLayersVisibility(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsVisible);

	/**
	 * Toggles the DataLayer's visibility
	 *
	 * @param DataLayer	The DataLayer to affect
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void ToggleDataLayerVisibility(UDataLayerInstance* DataLayer);

	/**
	 * Toggles the visibility of all of the DataLayers
	 *
	 * @param	DataLayers	The DataLayers to affect
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void ToggleDataLayersVisibility(const TArray<UDataLayerInstance*>& DataLayers);

	/**
	 * Changes the DataLayer's IsLoadedInEditor flag to the provided state
	 *
	 * @param	DataLayer						The DataLayer to affect.
	 * @param	bIsLoadedInEditor				The new value of the flag IsLoadedInEditor.
	 *											If True, the Editor loading will consider this DataLayer to load or not an Actor part of this DataLayer.
	 *											An Actor will not be loaded in the Editor if all its DataLayers are not LoadedInEditor.
	 * @param	bIsFromUserChange				If this change originates from a user change or not.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SetDataLayerIsLoadedInEditor(UDataLayerInstance* DataLayer, const bool bIsLoadedInEditor, const bool bIsFromUserChange);

	/**
	 * Changes the IsLoadedInEditor flag of the DataLayers to the provided state
	 *
	 * @param	DataLayers						The DataLayers to affect
	 * @param	bIsLoadedInEditor				The new value of the flag IsLoadedInEditor.
	 *											If True, the Editor loading will consider this DataLayer to load or not an Actor part of this DataLayer.
	 *											An Actor will not be loaded in the Editor if all its DataLayers are not LoadedInEditor.
	 * @param	bIsFromUserChange				If this change originates from a user change or not.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SetDataLayersIsLoadedInEditor(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsLoadedInEditor, const bool bIsFromUserChange);

	/**
	 * Toggles the DataLayer's IsLoadedInEditor flag
	 *
	 * @param	DataLayer						The DataLayer to affect
	 * @param	bIsFromUserChange				If this change originates from a user change or not.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool ToggleDataLayerIsLoadedInEditor(UDataLayerInstance* DataLayer, const bool bIsFromUserChange);

	/**
	 * Toggles the IsLoadedInEditor flag of all of the DataLayers
	 *
	 * @param	DataLayers						The DataLayers to affect
	 * @param	bIsFromUserChange				If this change originates from a user change or not.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool ToggleDataLayersIsLoadedInEditor(const TArray<UDataLayerInstance*>& DataLayers, const bool bIsFromUserChange);

	/**
	 * Set the visibility of all DataLayers to true
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void MakeAllDataLayersVisible();

	/**
	 * Gets the UDataLayerInstance associated to the DataLayerAsset
	 *
	 * @param	DataLayerAsset	The DataLayerAsset associated to the UDataLayerInstance
	 * @return					The UDataLayerInstance of the provided DataLayerAsset
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API UDataLayerInstance* GetDataLayerInstance(const UDataLayerAsset* DataLayerAsset) const;

	/**
	 * Gets the UDataLayerInstances associated to the each DataLayerAssets
	 *
	 * @param	DataLayerAssets	The array of DataLayerAssets associated to UDataLayerInstances
	 * @return					The array of UDataLayerInstances corresponding to a DataLayerAsset in the DataLayerAssets array
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API TArray<UDataLayerInstance*> GetDataLayerInstances(const TArray<UDataLayerAsset*> DataLayerAssets) const;

	/**
	 * Gets the UDataLayerInstance Object of the DataLayer name
	 *
	 * @param	DataLayerName	The name of the DataLayer whose UDataLayerInstance Object is returned
	 * @return					The UDataLayerInstance Object of the provided DataLayer name
	 */
	UE_API UDataLayerInstance* GetDataLayerInstance(const FName& DataLayerInstanceName) const;

	/**
	 * Gets all known DataLayers and appends them to the provided array
	 *
	 * @param OutDataLayers[OUT] Output array to store all known DataLayers
	 */
	UE_API void AddAllDataLayersTo(TArray<TWeakObjectPtr<UDataLayerInstance>>& OutDataLayers) const;

	/**
	 * Creates a UDataLayerInstance Object
	 *
	 * @param	Parameters The Data Layer Instance creation parameters
	 * @return	The newly created UDataLayerInstance Object
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API UDataLayerInstance* CreateDataLayerInstance(const FDataLayerCreationParameters& Parameters);

	/**
	 * Sets a Parent DataLayer for a specified DataLayer
	 * 
	 *  @param DataLayer		The child DataLayer.
	 *  @param ParentDataLayer	The parent DataLayer.
	 * 
	 *  @return	true if succeeded, false if failed.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API bool SetParentDataLayer(UDataLayerInstance* DataLayer, UDataLayerInstance* ParentDataLayer);

	/**
	 * Sets a Parent DataLayer for a specified list of DataLayers
	 * 
	 *  @param DataLayers		The child DataLayers.
	 *  @param ParentDataLayer	The parent DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void SetParentDataLayerForDataLayers(const TArray<UDataLayerInstance*>& DataLayers, UDataLayerInstance* ParentDataLayer);

	/**
	 * Sets the initial runtime state of the DataLayers
	 *
	 * @param	DataLayer						The DataLayer to affect
	 * @param	InitialRuntimeState				The initial runtime state.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void SetDataLayerInitialRuntimeState(UDataLayerInstance* DataLayer, EDataLayerRuntimeState InitialRuntimeState);

	/**
	 * Sets the initial editor visibility state of the DataLayers
	 *
	 * @param	DataLayer						The DataLayer to affect
	 * @param	InitialRuntimeState				The initial editor visibility state.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void SetDataLayerIsInitiallyVisible(UDataLayerInstance* DataLayer, bool bIsInitiallyVisible);

	/**
	 * Deletes all of the provided DataLayers
	 *
	 * @param DataLayersToDelete	A valid list of DataLayer.
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void DeleteDataLayers(const TArray<UDataLayerInstance*>& DataLayersToDelete);

	/**
	 * Deletes the provided DataLayer
	 *
	 * @param DataLayerToDelete		A valid DataLayer
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API void DeleteDataLayer(UDataLayerInstance* DataLayerToDelete);

	/**
	 * Returns all Data Layers
	 */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	UE_API TArray<UDataLayerInstance*> GetAllDataLayers();

	/**
	* Resets user override settings of all DataLayers
	*/
	UE_API bool ResetUserSettings();

	/*
	 * Returns whether the DataLayer contains one of more actors that are part of the editor selection.
	 */
	bool DoesDataLayerContainSelectedActors(const UDataLayerInstance* DataLayer) const { return GetSelectedDataLayersFromEditorSelection().Contains(DataLayer); }

	/**
	* Whether there are any deprecated DataLayerInstance found
	*/
	UE_DEPRECATED(5.8, "Deprecated data layers should have been removed by now.")
	bool HasDeprecatedDataLayers() const { return false; }

	/**
	 * Assign new unique short name to DataLayerInstance if it supports it. 
	 */
	UE_API bool SetDataLayerShortName(UDataLayerInstance* InDataLayerInstance, const FString& InNewShortName);

	/**
	 * Move actors to the Actor Editor Context's Data Layers.
	 * 
	 * @return true if operation succeeded.
	 */
	UE_API bool ApplyActorEditorContextDataLayersToActors(const TArray<AActor*>& InActors);

	/** Returns whether the actor handle passes the data layers filter. */
	UE_API bool PassDataLayersFilter(UWorld* World, const FWorldPartitionHandle& ActorHandle);

	//~ Begin Deprecated

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(all, "Per-view Data Layer visibility was removed.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Per-view Data Layer visibility was removed."))
	void UpdateAllViewVisibility(UDEPRECATED_DataLayer* DataLayerThatChanged) {}

	UE_DEPRECATED(all, "Per-view Data Layer visibility was removed.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Per-view Data Layer visibility was removed."))
	void UpdateActorAllViewsVisibility(AActor* Actor) {}
		
	UE_DEPRECATED(all, "Use SetDataLayerIsLoadedInEditor() instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Use SetDataLayerIsLoadedInEditor instead"))
	bool SetDataLayerIsDynamicallyLoadedInEditor(UDEPRECATED_DataLayer* DataLayer, const bool bIsLoadedInEditor, const bool bIsFromUserChange) { return false; }
	
	UE_DEPRECATED(all, "Use SetDataLayersIsLoadedInEditor() instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Use SetDataLayersIsLoadedInEditor instead"))
	bool SetDataLayersIsDynamicallyLoadedInEditor(const TArray<UDEPRECATED_DataLayer*>& DataLayers, const bool bIsLoadedInEditor, const bool bIsFromUserChange) { return false;  }
	
	UE_DEPRECATED(all, "Use ToggleDataLayerIsLoadedInEditor() instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Use ToggleDataLayerIsLoadedInEditor instead"))
	bool ToggleDataLayerIsDynamicallyLoadedInEditor(UDEPRECATED_DataLayer* DataLayer, const bool bIsFromUserChange) { return false; }
	
	UE_DEPRECATED(all, "Use ToggleDataLayersIsLoadedInEditor() instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "Use ToggleDataLayersIsLoadedInEditor instead"))
	bool ToggleDataLayersIsDynamicallyLoadedInEditor(const TArray<UDEPRECATED_DataLayer*>& DataLayers, const bool bIsFromUserChange) { return false; }
	
	UE_DEPRECATED(all, "Use GetDataLayerInstance instead")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use GetDataLayerInstance instead"))
	UDataLayerInstance* GetDataLayer(const struct FActorDataLayer& ActorDataLayer) const { return nullptr; }

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(all, "Use GetDataLayerInstance instead")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use GetDataLayerInstance instead"))
	UDataLayerInstance* GetDataLayerFromLabel(const FName& DataLayerLabel) const { return nullptr; }

	UE_DEPRECATED(all, "Renaming is not permitted anymore")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Renaming is not permitted anymore"))
	bool RenameDataLayer(UDataLayerInstance* DataLayer, const FName& NewDataLayerLabel) { return false; }

	UE_DEPRECATED(all, "Use CreateDataLayerInstance with FDataLayerCreationParameters instead")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use CreateDataLayerInstance with FDataLayerCreationParameters instead"))
	UDataLayerInstance* CreateDataLayer(UDataLayerInstance* ParentDataLayer = nullptr) { return nullptr; }

	//~ End Deprecated

private:

	/**
	 *	Synchronizes an newly created Actor's DataLayers with the DataLayer system
	 *
	 *	@param	Actor	The actor to initialize
	 */
	UE_API void InitializeNewActorDataLayers(AActor* Actor);

	/**
	 * Find and return the selected actors.
	 *
	 * @return				The selected AActor's as a TArray.
	 */
	UE_API TArray<AActor*> GetSelectedActors() const;

	/**
	 * Get the current UWorld object.
	 *
	 * @return						The UWorld* object
	 */
	UE_API UWorld* GetWorld() const; // Fallback to GWorld

	/** Delegate handler for FEditorDelegates::MapChange. It internally calls DataLayerChanged.Broadcast. */
	UE_API void EditorMapChange();

	/** Delegate handler for FEditorDelegates::RefreshDataLayerBrowser. It internally calls UpdateAllActorsVisibility to refresh the actors of each DataLayer. */
	UE_API void EditorRefreshDataLayerBrowser();

	/** Delegate handler for FEditorDelegates::PostUndoRedo. It internally calls DataLayerChanged.Broadcast and UpdateAllActorsVisibility to refresh the actors of each DataLayer. */
	UE_API void PostUndoRedo();

	UE_API void UpdateRegisteredWorldDelegates();
	UE_API void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	UE_API void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);
	UE_API void OnLoadedActorAddedToLevel(AActor& InActor);
	UE_API void OnDataLayerEditorLoadingStateChanged(bool bIsFromUserChange);

	UE_API bool SetDataLayerIsLoadedInEditorInternal(UDataLayerInstance* DataLayer, const bool bIsLoadedInEditor, const bool bIsFromUserChange);

	UE_API void BroadcastActorDataLayersChanged(const TWeakObjectPtr<AActor>& ChangedActor);
	UE_API void BroadcastDataLayerChanged(const EDataLayerAction Action, const TWeakObjectPtr<const UDataLayerInstance>& ChangedDataLayer, const FName& ChangedProperty);
	UE_API void BroadcastDataLayerEditorLoadingStateChanged(bool bIsFromUserChange);
	UE_API void OnSelectionChanged();
	UE_API void RebuildSelectedDataLayersFromEditorSelection();
	UE_API const TSet<TWeakObjectPtr<const UDataLayerInstance>>& GetSelectedDataLayersFromEditorSelection() const;
	UE_API bool SetParentDataLayerForDataLayersInternal(const TArray<UDataLayerInstance*>& DataLayers, UDataLayerInstance* ParentDataLayer);

	UE_API bool UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports, ULevel* InLevel);

	UE_API bool IsActorValidForDataLayerForClasses(AActor* Actor, const TSet<TSubclassOf<UDataLayerInstance>>& DataLayerInstanceClasses);

	// External Data Layer methods
	UE_API void OnActorPreSpawnInitialization(AActor* InActor);
	UE_API void OnNewActorsPlaced(UObject* InObjToUse, const TArray<AActor*>& InPlacedActors);
	UE_API void OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor);
	UE_API void OnExternalDataLayerAssetRegistrationStateChanged(const UExternalDataLayerAsset* ExternalDataLayerAsset, EExternalDataLayerRegistrationState OldState, EExternalDataLayerRegistrationState NewState);
	UE_API TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> OnLevelEditorDragDropWorldSurrogateReferencingObject(UWorld* ReferencingWorld, const FSoftObjectPath& Object);
	UE_API void ApplyContext(AActor* InActor, bool bInForceTryApply = false, AActor* InReplacedActor = nullptr);
	UE_API const UExternalDataLayerInstance* GetActorSpawningExternalDataLayerInstance(AActor* InActor) const;
	static UE_API const UExternalDataLayerAsset* GetReferencingWorldSurrogateObjectForObject(UWorld* ReferencingWorld, const FSoftObjectPath& ObjectPath);
	UE_API bool MoveActorToDataLayers(AActor* InActor, const TArray<UDataLayerInstance*>& InDataLayerInstances);
	UE_API void MoveActorToExternalDataLayer(AActor* InActor, const UExternalDataLayerInstance* InExternalDataLayerInstance, bool bInNotifyFailureReason = true);
	UE_API bool ShouldHandleActor(AActor* InActor) const;

	/** Contains Data Layers that contain actors that are part of the editor selection */
	mutable TSet<TWeakObjectPtr<const UDataLayerInstance>> SelectedDataLayersFromEditorSelection;

	/** Internal flag to know if SelectedDataLayersFromEditorSelection needs to be rebuilt. */
	mutable bool bRebuildSelectedDataLayersFromEditorSelection;

	/** When true, next Tick will call BroadcastDataLayerChanged */
	bool bAsyncBroadcastDataLayerChanged;

	/** When true, next Tick will call UpdateAllActorsVisibility */
	bool bAsyncUpdateAllActorsVisibility;

	/** When true, next Tick will force invalid editing viewports */
	bool bAsyncInvalidateViewports;

	/** Fires whenever one or more DataLayer changes */
	FOnDataLayerChanged DataLayerChanged;

	/**	Fires whenever one or more actor DataLayer changes */
	FOnActorDataLayersChanged ActorDataLayersChanged;

	/** Fires whenever one or more DataLayer editor loading state changed */
	FOnActorDataLayersEditorLoadingStateChanged DataLayerEditorLoadingStateChanged;

	FDelegateHandle OnActorDataLayersEditorLoadingStateChangedEngineBridgeHandle;

	/** Auxiliary class that sets the callback functions for multiple delegates */
	TSharedPtr<class FDataLayersBroadcast> DataLayersBroadcast;

	/** Delegate used to notify changes to ActorEditorContextSubsystem */
	FOnActorEditorContextClientChanged ActorEditorContextClientChanged;

	/** Last World to have registered world delegates */
	TWeakObjectPtr<UWorld> LastRegisteredWorldDelegates;

	/** Delegate handle for world's AddOnActorPreSpawnInitialization */
	FDelegateHandle OnActorPreSpawnInitializationDelegate;

	/** Last pushed warning to be send to the notification manager at next tick */
	TOptional<FText> LastWarningNotification;

	friend class FDataLayersBroadcast;
	friend class UContentBundleEditingSubmodule;
	friend struct FExternalDataLayerWorldSurrogateReferencingObject;
	friend class WorldPartitionTests::FExternalDataLayerTest;
};

template<class DataLayerInstanceType, typename ...Args>
DataLayerInstanceType* UDataLayerEditorSubsystem::CreateDataLayerInstance(AWorldDataLayers* WorldDataLayers, Args&&... InArgs)
{
	UDataLayerInstance* NewDataLayer = nullptr;

	if (WorldDataLayers)
	{
		NewDataLayer = WorldDataLayers->CreateDataLayer<DataLayerInstanceType>(Forward<Args>(InArgs)...);
	}

	if (NewDataLayer != nullptr)
	{
		BroadcastDataLayerChanged(EDataLayerAction::Add, NewDataLayer, NAME_None);
	}

	return CastChecked<DataLayerInstanceType>(NewDataLayer);
}

#undef UE_API
