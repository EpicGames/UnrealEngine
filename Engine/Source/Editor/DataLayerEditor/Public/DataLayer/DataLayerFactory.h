// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DataLayerFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(hidecategories = Object, MinimalAPI)
class UDataLayerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// Begin UFactory Interface	
};