// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API DATALAYEREDITOR_API

class SWidget;
class UDataLayerInstance;

struct FDataLayerPropertyTypeCustomizationHelper
{
	static UE_API TSharedRef<SWidget> CreateDataLayerMenu(TFunction<void(const UDataLayerInstance* DataLayer)> OnDataLayerSelectedFunction, TFunction<bool(const UDataLayerInstance*)> OnShouldFilterDataLayerInstanceFunction = [](const UDataLayerInstance*){ return false; });
};

#undef UE_API
