// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "ExternalDataLayerFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class UExternalDataLayerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual bool CanCreateNew() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// Begin UFactory Interface	
};