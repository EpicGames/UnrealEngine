// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Templates/SharedPointer.h"
#include "WorldPartition/DataLayer/IDataLayerEditorModule.h"

class AActor;
class FExtender;
class FUICommandList;
class SWidget;
class UToolMenu;
class UDataLayerInstance;
class UExternalDataLayerAsset;
class UExternalDataLayerInstance;

/**
 * The module holding all of the UI related pieces for DataLayers
 */
class FDataLayerEditorModule : public IDataLayerEditorModule
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates a DataLayer Browser widget
	 */
	virtual TSharedRef<class SWidget> CreateDataLayerBrowser();

	/*
	 * Selected DataLayer in DataLayer Browser widget
	 */
	virtual void SyncDataLayerBrowserToDataLayer(const UDataLayerInstance* DataLayer);

	/** Delegates to be called to extend the DataLayers menus */
	typedef TPair<TWeakObjectPtr<const UDataLayerInstance>, TWeakObjectPtr<const AActor>> FDataLayerActor;
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedRef<FExtender>, FDataLayersMenuExtender, const TSharedRef<FUICommandList>, const TSet<TWeakObjectPtr<const UDataLayerInstance>>& /*SelectedDataLayers*/,  const TSet<FDataLayerActor>& /*SelectedDataLayerActors*/);
	virtual TArray<FDataLayersMenuExtender>& GetAllDataLayersMenuExtenders() { return DataLayersMenuExtenders; }

	/* Implement IDataLayerEditorModule */
	virtual bool AddActorToDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers) override;
	virtual void SetActorEditorContextCurrentExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset) override;

private:
	void RegisterMenus();
	void MoveToExternalDataLayerMenu(UToolMenu* Menu);
	bool MoveActorsToExternalDataLayer(const TArray<AActor*>& InSelectedActors, const UExternalDataLayerInstance* InExternalDataLayerInstance, FText* OutReason = nullptr);
	TArray<AActor*> GetSelectedActors() const;

	TWeakPtr<SWidget> DataLayerBrowser;

	/** All extender delegates for the DataLayers menus */
	TArray<FDataLayersMenuExtender> DataLayersMenuExtenders;
};


